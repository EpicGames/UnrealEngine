// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DesktopPlatform : ModuleRules
{
	public DesktopPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DerivedDataCache",
				"SlateFontDialog",
				"SlateFileDialogs",
				"LauncherPlatform"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ApplicationCore",
				"Json"
			}
		);
		
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			if(Target.Type == TargetType.Editor)
			{
				DynamicallyLoadedModuleNames.Add("SlateFontDialog");
			}

			if (Target.bCompileAgainstCoreUObject && Target.bUsesSlate)
			{
				DynamicallyLoadedModuleNames.Add("SlateFileDialogs");
			}

			AddEngineThirdPartyPrivateStaticDependencies(Target, "SDL3");
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
