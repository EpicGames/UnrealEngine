// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

#define UE_API DESKTOPPLATFORM_API

struct FLockFile
{
	static UE_API bool TryReadAndClear(const TCHAR* FileName, FString& OutContents);
};

#undef UE_API
