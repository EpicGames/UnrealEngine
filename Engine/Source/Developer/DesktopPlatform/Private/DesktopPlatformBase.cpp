// Copyright Epic Games, Inc. All Rights Reserved.

#include "DesktopPlatformBase.h"
#include "HAL/FileManager.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/ScopeExit.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "String/LexFromString.h"
#include "Modules/ModuleManager.h"
#include "DesktopPlatformPrivate.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/SecureHash.h"
#include "String/RemoveFrom.h"

#define LOCTEXT_NAMESPACE "DesktopPlatform"


FDesktopPlatformBase::FDesktopPlatformBase()
{
	LauncherInstallationTimestamp = FDateTime::MinValue();
}

FString FDesktopPlatformBase::GetEngineDescription(const FString& Identifier)
{
	// Official release versions just have a version number
	if(IsStockEngineRelease(Identifier))
	{
		return Identifier;
	}

	// Otherwise get the path
	FString RootDir;
	if(!GetEngineRootDirFromIdentifier(Identifier, RootDir))
	{
		return FString();
	}

	// Convert it to a platform directory
	FString PlatformRootDir = RootDir;
	FPaths::MakePlatformFilename(PlatformRootDir);

	// Perforce build
	if (IsSourceDistribution(RootDir))
	{
		return FString::Printf(TEXT("Source build at %s"), *PlatformRootDir);
	}
	else
	{
		return FString::Printf(TEXT("Binary build at %s"), *PlatformRootDir);
	}
}

FString FDesktopPlatformBase::GetCurrentEngineIdentifier()
{
	if(CurrentEngineIdentifier.Len() == 0 && !GetEngineIdentifierFromRootDir(FPlatformMisc::RootDir(), CurrentEngineIdentifier))
	{
		CurrentEngineIdentifier.Empty();
	}
	return CurrentEngineIdentifier;
}

void FDesktopPlatformBase::EnumerateLauncherEngineInstallations(TMap<FString, FString> &OutInstallations)
{
	// Cache the launcher install list if necessary
	ReadLauncherInstallationList();

	// We've got a list of launcher installations. Filter it by the engine installations.
	for(TMap<FString, FString>::TConstIterator Iter(LauncherInstallationList); Iter; ++Iter)
	{
		FString AppName = Iter.Key();
		if(AppName.RemoveFromStart(TEXT("UE_"), ESearchCase::CaseSensitive))
		{
			OutInstallations.Add(AppName, Iter.Value());
		}
	}
}

void FDesktopPlatformBase::EnumerateLauncherSampleInstallations(TArray<FString> &OutInstallations)
{
	// Cache the launcher install list if necessary
	ReadLauncherInstallationList();

	// We've got a list of launcher installations. Filter it by the engine installations.
	for(TMap<FString, FString>::TConstIterator Iter(LauncherInstallationList); Iter; ++Iter)
	{
		FString AppName = Iter.Key();
		if(!AppName.StartsWith(TEXT("UE_"), ESearchCase::CaseSensitive))
		{
			OutInstallations.Add(Iter.Value());
		}
	}
}

void FDesktopPlatformBase::EnumerateLauncherSampleProjects(TArray<FString> &OutFileNames)
{
	// Enumerate all the sample installation directories
	TArray<FString> LauncherSampleDirectories;
	EnumerateLauncherSampleInstallations(LauncherSampleDirectories);

	// Find all the project files within them
	for(int32 Idx = 0; Idx < LauncherSampleDirectories.Num(); Idx++)
	{
		TArray<FString> FileNames;
		IFileManager::Get().FindFiles(FileNames, *(LauncherSampleDirectories[Idx] / TEXT("*.uproject")), true, false);
		OutFileNames.Append(FileNames);
	}
}

bool FDesktopPlatformBase::GetEngineRootDirFromIdentifier(const FString &Identifier, FString &OutRootDir)
{
	// Get all the installations
	TMap<FString, FString> Installations;
	EnumerateEngineInstallations(Installations);

	// Find the one with the right identifier
	for (TMap<FString, FString>::TConstIterator Iter(Installations); Iter; ++Iter)
	{
		if (Iter->Key == Identifier)
		{
			OutRootDir = Iter->Value;
			return true;
		}
	}
	return false;
}

bool FDesktopPlatformBase::GetEngineIdentifierFromRootDir(const FString &RootDir, FString &OutIdentifier)
{
	// Get all the installations
	TMap<FString, FString> Installations;
	EnumerateEngineInstallations(Installations);

	// Normalize the root directory
	FString NormalizedRootDir = RootDir;
	FPaths::CollapseRelativeDirectories(NormalizedRootDir);
	FPaths::NormalizeDirectoryName(NormalizedRootDir);

	// Find the label for the given directory
	for (TMap<FString, FString>::TConstIterator Iter(Installations); Iter; ++Iter)
	{
		if (Iter->Value == NormalizedRootDir)
		{
			OutIdentifier = Iter->Key;
			return true;
		}
	}

	// Otherwise just try to add it
	return RegisterEngineInstallation(NormalizedRootDir, OutIdentifier);
}

bool FDesktopPlatformBase::GetDefaultEngineIdentifier(FString &OutId)
{
	TMap<FString, FString> Installations;
	EnumerateEngineInstallations(Installations);

	bool bRes = false;
	if (Installations.Num() > 0)
	{
		// Default to the first install
		TMap<FString, FString>::TConstIterator Iter(Installations);
		OutId = Iter.Key();
		++Iter;

		// Try to find the most preferred install
		for(; Iter; ++Iter)
		{
			if(IsPreferredEngineIdentifier(Iter.Key(), OutId))
			{
				OutId = Iter.Key();
			}
		}
	}
	return bRes;
}

bool FDesktopPlatformBase::GetDefaultEngineRootDir(FString &OutDirName)
{
	FString Identifier;
	return GetDefaultEngineIdentifier(Identifier) && GetEngineRootDirFromIdentifier(Identifier, OutDirName);
}

bool FDesktopPlatformBase::IsPreferredEngineIdentifier(const FString &Identifier, const FString &OtherIdentifier)
{
	int32 Version = ParseReleaseVersion(Identifier);
	int32 OtherVersion = ParseReleaseVersion(OtherIdentifier);

	if(Version != OtherVersion)
	{
		return Version > OtherVersion;
	}
	else
	{
		return Identifier > OtherIdentifier;
	}
}

bool FDesktopPlatformBase::TryGetEngineVersion(const FString& RootDir, FEngineVersion& OutVersion)
{
	// Read the file to a string
	FString VersionText;
	if(FFileHelper::LoadFileToString(VersionText, *(RootDir / TEXT("Engine/Build/Build.version"))))
	{
		// Deserialize a JSON object from the string
		TSharedPtr< FJsonObject > Object;
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(VersionText);
		if(FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid())
		{
			int32 MajorVersion = 0;
			int32 MinorVersion = 0;
			int32 PatchVersion = 0;
			if(Object->TryGetNumberField(TEXT("MajorVersion"), MajorVersion) && Object->TryGetNumberField(TEXT("MinorVersion"), MinorVersion) && Object->TryGetNumberField(TEXT("PatchVersion"), PatchVersion))
			{
				int32 Changelist = 0;
				if(!Object->TryGetNumberField(TEXT("Changelist"), Changelist))
				{
					Changelist = 0;
				}

				int32 IsLicenseeVersion = 0;
				if(!Object->TryGetNumberField(TEXT("IsLicenseeVersion"), IsLicenseeVersion))
				{
					IsLicenseeVersion = 0;
				}

				FString BranchName;
				if(!Object->TryGetStringField(TEXT("BranchName"), BranchName))
				{
					BranchName = FString();
				}

				int EncodedChangelist = (IsLicenseeVersion == 0)? Changelist : FEngineVersionBase::EncodeLicenseeChangelist(Changelist);
				OutVersion = FEngineVersion(IntCastChecked<uint16>(MajorVersion), IntCastChecked<uint16>(MinorVersion), IntCastChecked<uint16>(PatchVersion), EncodedChangelist, BranchName);
				return true;
			}
		}
	}

	// Try to read the version file
	FString VersionHeader;
	if(FFileHelper::LoadFileToString(VersionHeader, *(RootDir / TEXT("Engine/Source/Runtime/Launch/Resources/Version.h"))))
	{
		int32 MajorVersion = -1;
		int32 MinorVersion = -1;
		int32 PatchVersion = -1;
		int32 Changelist = 0;
		int32 IsLicenseeVersion = 0;
		FString BranchName;

		// Scan the file for version defines
		const TCHAR* TextPos = *VersionHeader;
		while(*TextPos)
		{
			// Skip over any newlines
			while(FChar::IsWhitespace(*TextPos))
			{
				TextPos++;
			}

			// Buffer up a line of tokens
			TArray<FString> Tokens;
			while(*TextPos != '\n' && *TextPos != 0)
			{
				if(*TextPos == ' ' || *TextPos == '\t' || *TextPos == '\r')
				{
					// Skip over whitespace
					TextPos++;
				}
				else if(FChar::IsIdentifier(*TextPos))
				{
					// Parse an identifier. Exact C rules for an identifier don't really matter; we just need alphanumeric sequences.
					const TCHAR* TokenStart = TextPos++;
					while(FChar::IsIdentifier(*TextPos)) TextPos++;
					Tokens.Add(FString::ConstructFromPtrSize(TokenStart, UE_PTRDIFF_TO_INT32(TextPos - TokenStart)));
				}
				else if(*TextPos == '\"')
				{
					// Parse a string
					const TCHAR* TokenStart = TextPos++;
					while(*TextPos != 0 && (TextPos == TokenStart + 1 || *(TextPos - 1) != '\"')) TextPos++;
					Tokens.Add(FString::ConstructFromPtrSize(TokenStart, UE_PTRDIFF_TO_INT32(TextPos - TokenStart)));
				}
				else if(*TextPos == '/' && *(TextPos + 1) == '/')
				{
					// Skip a C++ style comment
					TextPos += 2;
					while(*TextPos != '\n' && *TextPos != 0) TextPos++;
				}
				else if(*TextPos == '/' && *(TextPos + 1) == '*' && *(TextPos + 2) != 0 && *(TextPos + 3) != 0)
				{
					// Skip a C-style comment
					TextPos += 4;
					while(*TextPos != 0 && (*(TextPos - 2) != '*' || *(TextPos - 1) != '/')) TextPos++;
				}
				else
				{
					// Take a single symbol character
					Tokens.Add(FString::ConstructFromPtrSize(TextPos, 1));
					TextPos++;
				}
			}

			// Check if it matches any version defines
			if(Tokens.Num() >= 4 && Tokens[0] == "#" && Tokens[1] == "define")
			{
				if(FChar::IsDigit(Tokens[3][0]))
				{
					if(Tokens[2] == "ENGINE_MAJOR_VERSION")
					{
						MajorVersion = FCString::Atoi(*Tokens[3]);
					}
					else if(Tokens[2] == "ENGINE_MINOR_VERSION")
					{
						MinorVersion = FCString::Atoi(*Tokens[3]);
					}
					else if(Tokens[2] == "ENGINE_PATCH_VERSION")
					{
						PatchVersion = FCString::Atoi(*Tokens[3]);
					}
					else if(Tokens[2] == "BUILT_FROM_CHANGELIST")
					{
						Changelist = FCString::Atoi(*Tokens[3]);
					}
					else if(Tokens[2] == "ENGINE_IS_LICENSEE_VERSION")
					{
						IsLicenseeVersion = FCString::Atoi(*Tokens[3]);
					}
				}
				else if(Tokens[3].StartsWith("\"") && Tokens[3].EndsWith("\""))
				{
					if(Tokens[2] == "BRANCH_NAME")
					{
						BranchName = Tokens[3].TrimQuotes();
					}
				}
			}
		}

		// If we have everything we need, fill in the version struct
		if(MajorVersion != -1 && MinorVersion != -1 && PatchVersion != -1)
		{
			int EncodedChangelist = (IsLicenseeVersion == 0)? Changelist : FEngineVersionBase::EncodeLicenseeChangelist(Changelist);
			OutVersion = FEngineVersion(IntCastChecked<uint16>(MajorVersion), IntCastChecked<uint16>(MinorVersion), IntCastChecked<uint16>(PatchVersion), EncodedChangelist, BranchName);
			return true;
		}
	}

	return false;
}

bool FDesktopPlatformBase::IsStockEngineRelease(const FString &Identifier)
{
	FGuid Guid;
	return !FGuid::Parse(Identifier, Guid);
}

bool FDesktopPlatformBase::TryParseStockEngineVersion(const FString& Identifier, FEngineVersion& OutVersion)
{
	TCHAR* End;

	uint64 Major = FCString::Strtoui64(*Identifier, &End, 10);
	if (Major > MAX_uint16 || *(End++) != '.')
	{
		return false;
	}

	uint64 Minor = FCString::Strtoui64(End, &End, 10);
	if (Minor > MAX_uint16 || *End != 0)
	{
		return false;
	}

	OutVersion = FEngineVersion(IntCastChecked<uint16>(Major), IntCastChecked<uint16>(Minor), 0, 0, TEXT(""));
	return true;
}

bool FDesktopPlatformBase::IsSourceDistribution(const FString &EngineRootDir)
{
	// Check for the existence of a SourceBuild.txt file
	FString SourceBuildPath = EngineRootDir / TEXT("Engine/Build/SourceDistribution.txt");
	return (IFileManager::Get().FileSize(*SourceBuildPath) >= 0);
}

bool FDesktopPlatformBase::IsPerforceBuild(const FString &EngineRootDir)
{
	// Check for the existence of a SourceBuild.txt file
	FString PerforceBuildPath = EngineRootDir / TEXT("Engine/Build/PerforceBuild.txt");
	return (IFileManager::Get().FileSize(*PerforceBuildPath) >= 0);
}

bool FDesktopPlatformBase::IsValidRootDirectory(const FString &RootDir)
{
	// Check that there's an Engine\Binaries directory underneath the root
	FString EngineBinariesDirName = RootDir / TEXT("Engine/Binaries");
	FPaths::NormalizeDirectoryName(EngineBinariesDirName);
	if(!IFileManager::Get().DirectoryExists(*EngineBinariesDirName))
	{
		return false;
	}

	// Also check there's an Engine\Build directory. This will filter out anything that has an engine-like directory structure but doesn't allow building code projects - like the launcher.
	FString EngineBuildDirName = RootDir / TEXT("Engine/Build");
	FPaths::NormalizeDirectoryName(EngineBuildDirName);
	if(!IFileManager::Get().DirectoryExists(*EngineBuildDirName))
	{
		return false;
	}

	// Otherwise it's valid
	return true;
}

bool FDesktopPlatformBase::SetEngineIdentifierForProject(const FString &ProjectFileName, const FString &InIdentifier)
{
	// Load the project file
	TSharedPtr<FJsonObject> ProjectFile = LoadProjectFile(ProjectFileName);
	if (!ProjectFile.IsValid())
	{
		return false;
	}

	// Check if the project is a non-foreign project of the given engine installation. If so, blank the identifier 
	// string to allow portability between source control databases. GetEngineIdentifierForProject will translate
	// the association back into a local identifier on other machines or syncs.
	FString Identifier = InIdentifier;
	if(Identifier.Len() > 0)
	{
		FString RootDir;
		if(GetEngineRootDirFromIdentifier(Identifier, RootDir))
		{
			const FUProjectDictionary &Dictionary = GetCachedProjectDictionary(RootDir);
			if(!Dictionary.IsForeignProject(ProjectFileName))
			{
				Identifier.Empty();
			}
		}
	}

	// Set the association on the project and save it
	ProjectFile->SetStringField(TEXT("EngineAssociation"), Identifier);
	return SaveProjectFile(ProjectFileName, ProjectFile);
}

bool FDesktopPlatformBase::GetEngineIdentifierForProject(const FString& ProjectFileName, FString& OutIdentifier)
{
	OutIdentifier.Empty();

	// Load the project file
	TSharedPtr<FJsonObject> ProjectFile = LoadProjectFile(ProjectFileName);
	if(!ProjectFile.IsValid())
	{
		return false;
	}

	// Try to read the identifier from it
	TSharedPtr<FJsonValue> Value = ProjectFile->TryGetField(TEXT("EngineAssociation"));
	if(Value.IsValid() && Value->Type == EJson::String)
	{
		OutIdentifier = Value->AsString();
		if(OutIdentifier.Len() > 0)
		{
			// If it's a path, convert it into an engine identifier
			if(OutIdentifier.Contains(TEXT("/")) || OutIdentifier.Contains("\\"))
			{
				FString EngineRootDir = FPaths::ConvertRelativePathToFull(FPaths::GetPath(ProjectFileName), OutIdentifier);
				if(!GetEngineIdentifierFromRootDir(EngineRootDir, OutIdentifier))
				{
					return false;
				}
			}
			return true;
		}
	}

	// Otherwise scan up through the directory hierarchy to find an installation
	FString ParentDir = FPaths::GetPath(ProjectFileName);
	FPaths::NormalizeDirectoryName(ParentDir);

	// Keep going until we reach the root
	int32 SeparatorIdx;
	while(ParentDir.FindLastChar(TEXT('/'), SeparatorIdx))
	{
		ParentDir.RemoveAt(SeparatorIdx, ParentDir.Len() - SeparatorIdx);
		if(IsValidRootDirectory(ParentDir) && GetEngineIdentifierFromRootDir(ParentDir, OutIdentifier))
		{
			return true;
		}
	}

	// Otherwise check the engine version string for 4.0, in case this project existed before the engine association stuff went in
	FString EngineVersionString;
	if(ProjectFile->TryGetStringField(TEXT("EngineVersion"), EngineVersionString) && EngineVersionString.Len() > 0)
	{
		FEngineVersion EngineVersion;
		if(FEngineVersion::Parse(EngineVersionString, EngineVersion) && EngineVersion.HasChangelist() && EngineVersion.ToString(EVersionComponent::Minor) == TEXT("4.0"))
		{
			OutIdentifier = TEXT("4.0");
			return true;
		}
	}

	return false;
}

bool FDesktopPlatformBase::OpenProject(const FString& ProjectFileName)
{
	FPlatformProcess::LaunchFileInDefaultExternalApplication(*ProjectFileName);
	return true;
}

bool FDesktopPlatformBase::CleanGameProject(const FString& ProjectDir, FString& OutFailPath, FFeedbackContext* Warn)
{
	// Begin a task
	Warn->BeginSlowTask(LOCTEXT("CleaningProject", "Removing stale build products..."), true);

	// Enumerate all the files
	TArray<FString> FileNames;
	TArray<FString> DirectoryNames;
	GetProjectBuildProducts(ProjectDir, FileNames, DirectoryNames);

	// Remove all the files
	for(int32 Idx = 0; Idx < FileNames.Num(); Idx++)
	{
		// Remove the file
		if(!IFileManager::Get().Delete(*FileNames[Idx]))
		{
			OutFailPath = FileNames[Idx];
			Warn->EndSlowTask();
			return false;
		}

		// Update the progress
		Warn->UpdateProgress(Idx, FileNames.Num() + DirectoryNames.Num());
	}

	// Remove all the directories
	for(int32 Idx = 0; Idx < DirectoryNames.Num(); Idx++)
	{
		// Remove the directory
		if(!IFileManager::Get().DeleteDirectory(*DirectoryNames[Idx], false, true))
		{
			OutFailPath = DirectoryNames[Idx];
			Warn->EndSlowTask();
			return false;
		}

		// Update the progress
		Warn->UpdateProgress(Idx + FileNames.Num(), FileNames.Num() + DirectoryNames.Num());
	}

	// End the task
	Warn->EndSlowTask();
	return true;
}

bool FDesktopPlatformBase::CompileGameProject(const FString& RootDir, const FString& ProjectFileName, FFeedbackContext* Warn, ECompilationResult::Type* OutResult)
{
	FModuleManager& ModuleManager = FModuleManager::Get();

	// Build the argument list
	FString Arguments = FString::Printf(TEXT("%s %s"), ModuleManager.GetUBTConfiguration(), FPlatformMisc::GetUBTPlatform());

	// Append the project name if it's a foreign project. Otherwise compile UnrealEditor.
	if ( ProjectFileName.IsEmpty() )
	{
		Arguments = TEXT("UnrealEditor ") + Arguments;
	}
	else
	{
		Arguments += FString::Printf(TEXT(" -Project=\"%s\" -TargetType=Editor"), *IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProjectFileName));
	}

	// Append any other options
	Arguments += " -Progress -NoEngineChanges -NoHotReloadFromIDE";

	// Run UBT
	int32 ExitCode;
	bool bResult = static_cast<IDesktopPlatform*>(this)->RunUnrealBuildTool(LOCTEXT("CompilingProject", "Compiling project..."), RootDir, Arguments, Warn, ExitCode);
	if (OutResult != nullptr)
	{
		*OutResult = (ECompilationResult::Type)ExitCode;
	}

	// Reset module paths in case they have changed during compilation
	ModuleManager.ResetModulePathsCache();

	return bResult;
}

bool FDesktopPlatformBase::GenerateProjectFiles(const FString& RootDir, const FString& ProjectFileName, FFeedbackContext* Warn, FString LogFilePath)
{
	FString Arguments = TEXT(" -projectfiles");

	// Build the arguments to pass to UBT. If it's a non-foreign project, just build full project files.
	if ( !ProjectFileName.IsEmpty() && GetCachedProjectDictionary(RootDir).IsForeignProject(ProjectFileName) )
	{
		// Figure out whether it's a foreign project
		const FUProjectDictionary &ProjectDictionary = GetCachedProjectDictionary(RootDir);
		if(ProjectDictionary.IsForeignProject(ProjectFileName))
		{
			Arguments += FString::Printf(TEXT(" -project=\"%s\""), *IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProjectFileName));

			// Always include game source
			Arguments += TEXT(" -game");

			// Determine whether or not to include engine source
			if(IsSourceDistribution(RootDir))
			{
				Arguments += TEXT(" -engine");
			}
			else
			{
				// If this is used within UnrealVersionSelector then we still need to pass
				// -rocket to deal with old versions that don't use Rocket.txt file
				Arguments += TEXT(" -rocket");
			}
		}
	}
	Arguments += TEXT(" -progress");

	if (!LogFilePath.IsEmpty())
	{
		Arguments += FString::Printf(TEXT(" -log=\"%s\""), *LogFilePath);
	}

	// Compile UnrealBuildTool if it doesn't exist. This can happen if we're just copying source from somewhere.
	bool bRes = true;
	Warn->BeginSlowTask(LOCTEXT("GeneratingProjectFiles", "Generating project files..."), true, true);
	if(!FPaths::FileExists(GetUnrealBuildToolExecutableFilename(RootDir)))
	{
		Warn->StatusUpdate(0, 1, LOCTEXT("BuildingUBT", "Building UnrealBuildTool..."));
		bRes = BuildUnrealBuildTool(RootDir, *Warn);
	}
	if(bRes)
	{
		Warn->StatusUpdate(0, 1, LOCTEXT("GeneratingProjectFiles", "Generating project files..."));
		bRes = RunUnrealBuildTool(LOCTEXT("GeneratingProjectFiles", "Generating project files..."), RootDir, Arguments, Warn);
	}
	Warn->EndSlowTask();
	return bRes;
}

bool FDesktopPlatformBase::IsUnrealBuildToolAvailable()
{
	// If using installed build and the unreal build tool executable exists, then UBT is available. Otherwise check it can be built.
	if (FApp::IsEngineInstalled())
	{
		return FPaths::FileExists(GetUnrealBuildToolExecutableFilename(FPaths::RootDir()));
	}
	else
	{
		return FPaths::FileExists(GetUnrealBuildToolProjectFileName(FPaths::RootDir()));
	}
}

bool FDesktopPlatformBase::InvokeUnrealBuildToolSync(const FString& InCmdLineParams, FOutputDevice &Ar, bool bSkipBuildUBT, int32& OutReturnCode, FString& OutProcOutput)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDesktopPlatformBase::InvokeUnrealBuildToolSync);
	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	bool bInvoked = false;
	FProcHandle ProcHandle = InvokeUnrealBuildToolAsync(InCmdLineParams, Ar, PipeRead, PipeWrite, bSkipBuildUBT);
	if (ProcHandle.IsValid())
	{
		// rather than waiting, we must flush the read pipe or UBT will stall if it writes out a ton of text to the console.
		while (FPlatformProcess::IsProcRunning(ProcHandle))
		{
			OutProcOutput += FPlatformProcess::ReadPipe(PipeRead);
			FPlatformProcess::Sleep(0.1f);
		}		
		bInvoked = true;
		bool bGotReturnCode = FPlatformProcess::GetProcReturnCode(ProcHandle, &OutReturnCode);		
		check(bGotReturnCode);
	}
	else
	{
		bInvoked = false;
		OutReturnCode = -1;
		OutProcOutput = TEXT("");
	}


	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
	if (ProcHandle.IsValid())
	{
		FPlatformProcess::CloseProc(ProcHandle);
	}

	return bInvoked;
}

FProcHandle FDesktopPlatformBase::InvokeUnrealBuildToolAsync(const FString& InCmdLineParams, FOutputDevice &Ar, void*& OutReadPipe, void*& OutWritePipe, bool bSkipBuildUBT)
{
	FString CmdLineParams = InCmdLineParams;

	// UnrealBuildTool is currently always located in the Binaries/DotNET folder
	FString ExecutableFileName = GetUnrealBuildToolExecutableFilename(FPaths::RootDir());

	// Installed builds never build UBT, UnrealBuildTool should already exist
	bool bSkipBuild = FApp::IsEngineInstalled() || bSkipBuildUBT;
	if (!bSkipBuild)
	{
		// When not using an installed build, we should attempt to build UBT to make sure it is up to date
		// Only do this if we have not already successfully done it once during this session.
		static bool bSuccessfullyBuiltUBTOnce = false;
		if (!bSuccessfullyBuiltUBTOnce)
		{
			Ar.Log(TEXT("Building UnrealBuildTool..."));
			if (BuildUnrealBuildTool(FPaths::RootDir(), Ar))
			{
				bSuccessfullyBuiltUBTOnce = true;
			}
			else
			{
				// Failed to build UBT
				Ar.Log(TEXT("Failed to build UnrealBuildTool."));
				UE_LOGF(LogDesktopPlatform, Warning, "Failed to compile UnrealBuildTool (project file is '%ls', exe path is '%ls')", *GetUnrealBuildToolProjectFileName(FPaths::RootDir()), *ExecutableFileName);
				return FProcHandle();
			}
		}
	}

#if PLATFORM_LINUX
	CmdLineParams += (" -progress");
#endif // PLATFORM_LINUX

	Ar.Logf(TEXT("Launching UnrealBuildTool... [%s %s]"), *ExecutableFileName, *CmdLineParams);

	// Run UnrealBuildTool
	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = bLaunchHidden;

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(*ExecutableFileName, *CmdLineParams, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, NULL, 0, NULL, OutWritePipe);
	if (!ProcHandle.IsValid())
	{
		UE_LOGF(LogDesktopPlatform, Warning, "Failed to launch UnrealBuildTool (exe path is '%ls')", *ExecutableFileName);
		Ar.Logf(TEXT("Failed to launch Unreal Build Tool. (%s)"), *ExecutableFileName);
	}

	return ProcHandle;
}

bool FDesktopPlatformBase::RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn)
{
	int32 ExitCode;
	return static_cast<IDesktopPlatform*>(this)->RunUnrealBuildTool(Description, RootDir, Arguments, Warn, ExitCode);
}

bool FDesktopPlatformBase::IsUnrealBuildToolRunning()
{
	FString RunsDir = FPaths::Combine(FPaths::EngineIntermediateDir(), TEXT("UbtRuns"));
	if (!FPaths::DirectoryExists(RunsDir))
	{
		return false;
	}

	bool bIsRunning = false;
	IFileManager::Get().IterateDirectory(*RunsDir, [&bIsRunning](const TCHAR* Pathname, bool bIsDirectory)
		{
			if (!bIsDirectory)
			{
				bool bDeleteFile = true;

				FString Filename = FPaths::GetBaseFilename(FString(Pathname));
				const TCHAR* Delim = FCString::Strchr(*Filename, '_');
				if (Delim != nullptr)
				{
					FStringView Pid(*Filename, UE_PTRDIFF_TO_INT32(Delim - *Filename));
					int ProcessId = 0;
					LexFromString(ProcessId, Pid);
					FString EntryFullPath = FPlatformProcess::GetApplicationName(ProcessId);
					if (!EntryFullPath.IsEmpty())
					{
						EntryFullPath.ToUpperInline();
						const auto Utf8String = StringCast<UTF8CHAR>(*EntryFullPath);
						FMD5Hash Hash;
						LexFromString(Hash, Delim + 1);

						FMD5 Md5Gen;
						Md5Gen.Update(reinterpret_cast<const uint8*>(Utf8String.Get()), Utf8String.Length());
						FMD5Hash TestHash;
						TestHash.Set(Md5Gen);
						if (Hash == TestHash)
						{
							bDeleteFile = false;
							bIsRunning = true;
						}
					}
					if (bDeleteFile)
					{
						IFileManager::Get().Delete(Pathname);
					}
				}
			}
			return true;
		});

	return bIsRunning;
}

bool FDesktopPlatformBase::GetOidcAccessToken(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin)
{
	FString BaseArguments;
	BaseArguments += FString::Printf(TEXT(" --Service=\"%s\""), *ProviderIdentifier);
	if (ProjectFileName.Len() > 0)
	{
		BaseArguments += FString::Printf(TEXT(" --project=\"%s\""), *IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::GetPath(*ProjectFileName)));
	}

	return GetOidcAccessTokenInternal(RootDir, BaseArguments, InteractionLevel, Warn, OutToken, OutTokenExpiresAt, bOutWasInteractiveLogin);
}

bool FDesktopPlatformBase::GetOidcAccessTokenWithRemoteConfig(const FString& RootDir, const FString& RemoteHost, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin)
{
	FString BaseArguments;
	BaseArguments += FString::Printf(TEXT(" --AuthConfigUrl=\"%s\""), *RemoteHost);

	return GetOidcAccessTokenInternal(RootDir, BaseArguments, InteractionLevel, Warn, OutToken, OutTokenExpiresAt, bOutWasInteractiveLogin);
}

bool FDesktopPlatformBase::GetOidcAccessToken(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin)
{
	const EDesktopLoginInteractionLevel InteractionLevel = bUnattended ? EDesktopLoginInteractionLevel::None : EDesktopLoginInteractionLevel::Interactive;
	return GetOidcAccessToken(RootDir, ProjectFileName, ProviderIdentifier, InteractionLevel, Warn, OutToken, OutTokenExpiresAt, bOutWasInteractiveLogin);
}

bool FDesktopPlatformBase::GetOidcAccessTokenWithRemoteConfig(const FString& RootDir, const FString& RemoteHost, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin)
{
	const EDesktopLoginInteractionLevel InteractionLevel = bUnattended ? EDesktopLoginInteractionLevel::None : EDesktopLoginInteractionLevel::Interactive;
	return GetOidcAccessTokenWithRemoteConfig(RootDir, RemoteHost, InteractionLevel, Warn, OutToken, OutTokenExpiresAt, bOutWasInteractiveLogin);
}

bool FDesktopPlatformBase::GetOidcTokenStatus(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, FFeedbackContext* Warn, int& OutStatus)
{
	IFileManager::Get().MakeDirectory(*FPaths::ProjectIntermediateDir(), /*bTree*/ true);
	FString ResultFilePath = FPaths::CreateTempFilename(*FPaths::ProjectIntermediateDir(), TEXT("oidcToken-status.json"));

	FString Arguments = TEXT(" ");
	Arguments += FString::Printf(TEXT(" --Service=\"%s\""), *ProviderIdentifier);
	Arguments += FString::Printf(TEXT(" --OutFile=\"%s\""), *ResultFilePath);
	Arguments += FString::Printf(TEXT(" --project=\"%s\""),  *IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::ProjectDir()));

	bool bRes = true;
	int32 ExitCode;
	FString ProcessStdout;
	bRes = InvokeOidcTokenToolSync(LOCTEXT("GetOidcAccessTokenStatus", "Fetching OIDC Access Token Status..."), RootDir, Arguments, Warn, ExitCode, ProcessStdout);

	if (!bRes)
	{
		UE_LOGF(LogDesktopPlatform, Warning, "Failed to run OidcToken to determine token status (project file is '%ls', exe path is '%ls')", *ProjectFileName, *GetOidcTokenExecutableFilename(RootDir));
		return false;
	}
	
	// Read the file to a string
	FString TokenText;
	if(FFileHelper::LoadFileToString(TokenText, *ResultFilePath))
	{
		// deserialize the json file
		TSharedPtr< FJsonObject > Object;
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(TokenText);
		if(FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid())
		{
			int Status;
			if(Object->TryGetNumberField(TEXT("Status"), Status))
			{
				OutStatus = Status;

				// Remove the output file if its still around
				IFileManager::Get().Delete(*ResultFilePath, true, false, true);

				return true;
			}
		}
	}

	// Remove the output file if its still around
	IFileManager::Get().Delete(*ResultFilePath, true, false, true);
	return false;
}

FString FDesktopPlatformBase::ReadHordeUrlWithoutCache(FString& OutHordeUrlConfigSource)
{
	// First try to read Horde URL from environment variable
	FString Url = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_URL"));
	if (!Url.IsEmpty())
	{
		OutHordeUrlConfigSource = TEXT("Environment variable \"UE_HORDE_URL\"");
		return Url;
	}

#if PLATFORM_WINDOWS
	// On Windows, try to read URL from registry entry next
	if (FWindowsPlatformMisc::QueryRegKey(HKEY_CURRENT_USER, TEXT("SOFTWARE\\Epic Games\\Horde"), TEXT("Url"), Url) && !Url.IsEmpty())
	{
		OutHordeUrlConfigSource = TEXT("Windows registry \"HKEY_CURRENT_USER\\SOFTWARE\\Epic Games\\Horde\"");
		return Url;
	}

	if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Epic Games\\Horde"), TEXT("Url"), Url) && !Url.IsEmpty())
	{
		OutHordeUrlConfigSource = TEXT("Windows registry \"HKEY_LOCAL_MACHINE\\SOFTWARE\\Epic Games\\Horde\"");
		return Url;
	}
#else
	// On POSIX, try to read URL from .horde.json user file next
	FString FileName = FPaths::Combine(FPlatformProcess::UserHomeDir(), TEXT(".horde.json"));

	FString FileContents;
	if (FFileHelper::LoadFileToString(FileContents, *FileName))
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContents);

		TSharedPtr<FJsonObject> Object;
		if (FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid() && Object->TryGetStringField(TEXT("Url"), Url))
		{
			OutHordeUrlConfigSource = FileName;
			return Url;
		}
	}
#endif

	// As last fallback, try to read Horde URL from INI configuration
	if (GConfig->GetString(TEXT("Horde"), TEXT("ServerUrl"), Url, GEngineIni))
	{
		OutHordeUrlConfigSource = FString::Printf(TEXT("%s INI configuration"), *GEngineIni);
		return Url;
	}

	OutHordeUrlConfigSource = TEXT("Not found");
	return FString();
}

bool FDesktopPlatformBase::GetHordeUrl(FString& OutHordeUrl, FString* OutHordeUrlConfigSource)
{
	if (!bInitializedHordeServerUrl)
	{
		HordeServerUrl = ReadHordeUrlWithoutCache(HordeServerUrlConfigSource);
		bInitializedHordeServerUrl = true;
	}

	OutHordeUrl = HordeServerUrl;
	if (OutHordeUrlConfigSource)
	{
		*OutHordeUrlConfigSource = HordeServerUrlConfigSource;
	}
	return !OutHordeUrl.IsEmpty();
}

void FDesktopPlatformBase::SetHordeUrl(const FString& HordeUrl)
{
	HordeServerUrl = HordeUrl;
	bInitializedHordeServerUrl = true;
	HordeServerUrlConfigSource = TEXT("Unspecified");

#if PLATFORM_WINDOWS
	if (HordeUrl.IsEmpty())
	{
		FPlatformMisc::DeleteStoredValue(TEXT("Epic Games"), TEXT("Horde"), TEXT("Url"));
	}
	else
	{
		FPlatformMisc::SetStoredValue(TEXT("Epic Games"), TEXT("Horde"), TEXT("Url"), HordeUrl);
	}
#else
	FString FileName = FPaths::Combine(FPlatformProcess::UserHomeDir(), TEXT(".horde.json"));

	// Read the existing json file
	TSharedPtr<FJsonObject> Object;

	FString FileContents;
	if (FFileHelper::LoadFileToString(FileContents, *FileName))
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContents);
		FJsonSerializer::Deserialize(Reader, Object);
	}

	// Update the object with the new URL
	if (!Object.IsValid())
	{
		Object = MakeShared<FJsonObject>();
	}
	Object->SetStringField(TEXT("Url"), HordeUrl);

	// Write it back out agai
	FString OutputFileContents;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputFileContents);
	if (FJsonSerializer::Serialize(Object.ToSharedRef(), *Writer))
	{
		FFileHelper::SaveStringToFile(OutputFileContents, *FileName);
	}
#endif
}

bool FDesktopPlatformBase::GetHordeAccessToken(const FString& HordeUrl, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin)
{
	FString Token = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_TOKEN"));
	if (!Token.IsEmpty())
	{
		FString Url = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_URL"));
		if (Url / TEXT("") == HordeUrl / TEXT(""))
		{
			OutToken = Token;
			OutTokenExpiresAt = FDateTime();
			return true;
		}
	}

	FString BaseArguments = FString::Printf(TEXT(" --HordeUrl=\"%s\""), *HordeUrl);
	return GetOidcAccessTokenInternal(FPaths::RootDir(), BaseArguments, InteractionLevel, Warn, OutToken, OutTokenExpiresAt, bOutWasInteractiveLogin);
}

bool FDesktopPlatformBase::GetHordeAccessToken(const FString& HordeUrl, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin)
{
	const EDesktopLoginInteractionLevel InteractionLevel = bUnattended ? EDesktopLoginInteractionLevel::None : EDesktopLoginInteractionLevel::Interactive;
	return GetHordeAccessToken(HordeUrl, InteractionLevel, Warn, OutToken, OutTokenExpiresAt, bOutWasInteractiveLogin);
}

bool FDesktopPlatformBase::GetOidcAccessTokenInternal(const FString& RootDir, const FString& BaseArguments, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin)
{
	#if !defined(UE_NO_ENGINE_OIDC) || UE_NO_ENGINE_OIDC == 0
	FString ResultFilePath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("oidcToken.json"));
	#else
	FString ResultFilePath = TEXT("oidcToken.json");
	#endif

	ON_SCOPE_EXIT
	{
		IFileManager::Get().Delete(*ResultFilePath, false /* RequireExists */, true /* EvenIfReadOnly */, true /* Quiet */);
	};

	FString Arguments = TEXT(" ") + BaseArguments;
	Arguments += FString::Printf(TEXT(" --OutFile=\"%s\""), *ResultFilePath);

	FString UnattendedArguments = Arguments;
	UnattendedArguments += TEXT(" --Unattended=true");

	// first we attempt to fetch a token using cached offline tokens, thus setting unattended
	bool bRes = true;
	int32 ExitCode;
	FString ProcessStdout;
	bRes = InvokeOidcTokenToolSync(LOCTEXT("GetOidcAccessToken", "Fetching OIDC Access Token..."), RootDir, UnattendedArguments, Warn, ExitCode, ProcessStdout);

	bOutWasInteractiveLogin = false;

	if (ExitCode == 10)
	{
		bRes = GetOidcAccessTokenInteractive(RootDir, Arguments, InteractionLevel, Warn, ExitCode);

		bOutWasInteractiveLogin = true;

		if (!bRes)
		{
			if (InteractionLevel != EDesktopLoginInteractionLevel::Interactive)
			{
				UE_LOGF(LogDesktopPlatform, Warning, "Unable to allocate an access token. Unable to complete interactive login. Make sure you start the editor and login once or log in using UGS or using the UGS cli command 'login'. (Ran '%ls%ls')", *GetOidcTokenExecutableFilename(RootDir), *Arguments);
			}
			else
			{
				UE_LOGF(LogDesktopPlatform, Error, "Unable to allocate an access token. Interactive login failed, make sure you are assigned access and are able to login in the created browser window. (Ran '%ls%ls')", *GetOidcTokenExecutableFilename(RootDir), *Arguments);
			}
			return false;
		}
	}

	if (!bRes)
	{
		UE_LOGF(LogDesktopPlatform, Warning, "Failed to run OidcToken ('%ls%ls'). ExitCode: %i", *GetOidcTokenExecutableFilename(RootDir), *Arguments, ExitCode);
		return false;
	}

	if (ExitCode == 11)
	{
		UE_LOGF(LogDesktopPlatform, Display, "Auth is disabled for connection");

		OutToken = FString();
		OutTokenExpiresAt = FDateTime::MaxValue();
	
		return true;
	}

	// Read the file to a string
	FString TokenText;
	if(FFileHelper::LoadFileToString(TokenText, *ResultFilePath))
	{
		// deserialize the json file
		TSharedPtr< FJsonObject > Object;
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(TokenText);
		if(FJsonSerializer::Deserialize(Reader, Object) && Object.IsValid())
		{
			FString Token;
			FString ExpiresAt;
			if(Object->TryGetStringField(TEXT("Token"), Token) && Object->TryGetStringField(TEXT("ExpiresAt"), ExpiresAt))
			{
				OutToken = Token;

				FDateTime::ParseIso8601(*ExpiresAt, OutTokenExpiresAt);

				// Remove the output file if its still around
				IFileManager::Get().Delete(*ResultFilePath, true, false, true);

				return true;
			}
		}
	}

	UE_LOGF(LogDesktopPlatform, Warning, "Failed to run OidcToken ('%ls%ls'). No result file found at '%ls', closed with exit code: %d", *GetOidcTokenExecutableFilename(RootDir), *Arguments, *ResultFilePath, ExitCode);

	// Remove the output file if its still around
	IFileManager::Get().Delete(*ResultFilePath, true, false, true);
	return false;
}

bool FDesktopPlatformBase::GetOidcAccessTokenInteractive(const FString& RootDir, const FString& Arguments, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, int32& OutReturnCode)
{
	if (InteractionLevel == EDesktopLoginInteractionLevel::None)
	{
		OutReturnCode = -1;
		return false;
	}

#if WITH_EDITOR
	check(GConfig && GConfig->IsReadyForUse());

	EAppReturnType::Type userAcknowledgedResult = EAppReturnType::Yes;
	bool bSkipInitialAcknowledgement = false;
	GConfig->GetBool(TEXT("/Script/UnrealEd.EditorSettings"), TEXT("InteractiveOidcWithoutAcknowledgement"), bSkipInitialAcknowledgement, GEditorSettingsIni);

	if (!bSkipInitialAcknowledgement)
	{
		if (InteractionLevel != EDesktopLoginInteractionLevel::Interactive)
		{
			OutReturnCode = -1;
			return false;
		}

		FText OidcInteractivePromptTitle = NSLOCTEXT("OidcToken", "OidcToken_InteractiveLaunchPromptTitle", "Unreal Engine - Authentication Required");
		FText OidcInteractiveLaunchPromptText = NSLOCTEXT("OidcToken", "OidcToken_InteractiveLaunch", "Your team's preferred DDC (Derived Data Cache) requires you to log in.\n\nClick Yes to open the authentication page in your web browser, or Yes To All to always proceed to browser authentication without a prompt.\n\nClick No to decline authentication and work with a different shared or local DDC instead. However, this may cause delays while the editor prepares the assets you need.");
		userAcknowledgedResult = FPlatformMisc::MessageBoxExt(EAppMsgType::YesNoYesAll, *OidcInteractiveLaunchPromptText.ToString(), *OidcInteractivePromptTitle.ToString());

		if (userAcknowledgedResult == EAppReturnType::YesAll)
		{
			GConfig->SetBool(TEXT("/Script/UnrealEd.EditorSettings"), TEXT("InteractiveOidcWithoutAcknowledgement"), true, GEditorSettingsIni);
			userAcknowledgedResult = EAppReturnType::Yes;
		}
	}

	if (userAcknowledgedResult != EAppReturnType::Yes)
	{
		OutReturnCode = -1;
		return false;
	}
#endif

	// user has acknowledged the login, we update the editor progress and then we run oidc token to spawn the browser and prompt the login

	FScopedSlowTask WaitForOidcTokenSlowTask(0, NSLOCTEXT("OidcToken", "OidcToken_WaitingForToken", "Waiting for OidcToken to finish login"));

	// run the oidc token app and wait for it to finish, prompting users if they have not logged in after a while
	OutReturnCode = 1;
	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	bool bInvoked = false;
	FProcHandle ProcHandle = InvokeOidcTokenToolAsync(Arguments, PipeRead, PipeWrite);

	if (ProcHandle.IsValid())
	{
		bInvoked = true;
	}

	uint64 WaitStartTime = FPlatformTime::Cycles64();
	enum class EWaitDurationPhase
	{
		Initial,
		Prompt,
		Waiting
	} DurationPhase = EWaitDurationPhase::Initial;
	bool bIsFinished = false;
	while (!bIsFinished)
	{
		// check if token app has finished running
		bIsFinished = !FPlatformProcess::IsProcRunning(ProcHandle);

		double WaitDuration = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - WaitStartTime);
		if (DurationPhase == EWaitDurationPhase::Initial)
		{
			// Note that the dialog may not show up when tokens are allocated early in the launch cycle, but this will at least ensure
			// the splash screen is refreshed with the appropriate text status message.
			WaitForOidcTokenSlowTask.MakeDialog(true, false);
			UE_LOGF(LogDesktopPlatform, Display, "Waiting for OidcToken to finish login...");
			DurationPhase = EWaitDurationPhase::Prompt;
		}
		// once we have waited for 30 seconds without success we give the user a option to abort, or if unattended, abort without prompting
		else if (WaitDuration > 30.0 && DurationPhase == EWaitDurationPhase::Prompt)
		{
			if (InteractionLevel == EDesktopLoginInteractionLevel::TimeLimited)
			{
				bIsFinished = !FPlatformProcess::IsProcRunning(ProcHandle);
				break;
			}

			FText OidcLongWaitPromptTitle = NSLOCTEXT("OidcToken", "OidcToken_LongWaitPromptTitle", "Wait for user login?");
			FText OidcLongWaitPromptText = NSLOCTEXT("OidcToken", "OidcToken_LongWaitPromptText", "Login is taking a long time, make sure you have entered your credentials in your browser window. It can be in a tab in an already existing window. Keep waiting?");
			if (FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *OidcLongWaitPromptText.ToString(), *OidcLongWaitPromptTitle.ToString()) == EAppReturnType::No)
			{
				bIsFinished = !FPlatformProcess::IsProcRunning(ProcHandle);
				break;
			}
			// change phase so we do not prompt the user again
			DurationPhase = EWaitDurationPhase::Waiting;
		}

		if (WaitForOidcTokenSlowTask.ShouldCancel())
		{
			bIsFinished = !FPlatformProcess::IsProcRunning(ProcHandle);
			break;
		}
		FPlatformProcess::Sleep(0.1f);
	}

	if (!bIsFinished)
	{
		OutReturnCode = -1;
		FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
		FPlatformProcess::TerminateProc(ProcHandle);
		if (ProcHandle.IsValid())
		{
			FPlatformProcess::CloseProc(ProcHandle);
		}

		return false;
	}

	if (!FPlatformProcess::GetProcReturnCode(ProcHandle, &OutReturnCode))
	{
		// OidcToken can throw exceptions, and in those cases on Mac & Linux, we don't
		// get a return code.  Windows always produces a return code.  Indicate it was invoked
		// but yield a return code of -1.
		OutReturnCode = -1;
	}
	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
	if (ProcHandle.IsValid())
	{
		FPlatformProcess::CloseProc(ProcHandle);
	}

	return bInvoked;
}

bool FDesktopPlatformBase::InvokeOidcTokenToolSync(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutReturnCode, FString& OutProcOutput)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDesktopPlatformBase::InvokeOidcTokenToolSync);
	OutReturnCode = 1;

	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	bool bInvoked = false;
	FProcHandle ProcHandle = InvokeOidcTokenToolAsync(Arguments, PipeRead, PipeWrite);
	if (ProcHandle.IsValid())
	{
		// rather than waiting, we must flush the read pipe or UBT will stall if it writes out a ton of text to the console.
		while (FPlatformProcess::IsProcRunning(ProcHandle))
		{
			OutProcOutput += FPlatformProcess::ReadPipe(PipeRead);
			FPlatformProcess::Sleep(0.1f);
		}		
		bInvoked = true;
		if (!FPlatformProcess::GetProcReturnCode(ProcHandle, &OutReturnCode))
		{
			// OidcToken can throw exceptions, and in those cases on Mac & Linux, we don't
			// get a return code.  Windows always produces a return code.  Indicate it was invoked
			// but yield a return code of -1.
			OutReturnCode = -1;
		}
	}
	else
	{
		bInvoked = false;
		OutReturnCode = -1;
		OutProcOutput = TEXT("");
	}


	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
	if (ProcHandle.IsValid())
	{
		FPlatformProcess::CloseProc(ProcHandle);
	}

	return bInvoked;
}

FProcHandle FDesktopPlatformBase::InvokeOidcTokenToolAsync(const FString& InArguments, void*& OutReadPipe, void*& OutWritePipe)
{
	FString CmdLineParams = InArguments;
	FString ExecutableFileName = GetOidcTokenExecutableFilename(FPaths::RootDir());
	UE_LOGF(LogDesktopPlatform, Display, "Launching OidcToken... [%ls %ls]", *ExecutableFileName, *CmdLineParams);

	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = bLaunchHidden;

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(*ExecutableFileName, *CmdLineParams, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, NULL, 0, FPlatformProcess::BaseDir(), OutWritePipe);
	if (!ProcHandle.IsValid())
	{
		UE_LOGF(LogDesktopPlatform, Warning, "Failed to launch OidcToken (exe path is '%ls')", *ExecutableFileName);
	}

	return ProcHandle;
}

struct FTargetFileVisitor : IPlatformFile::FDirectoryStatVisitor
{
	const TSet<FString>& OriginalTargetNames;
	TSet<FString>& RemainingTargetNames;
	FDateTime MaxDateTime;
	TArray<FString> SubDirectories;
	bool bSearchSubDirectories = true;
	bool bSearchPluginForTargets = false;
	bool bCacheInvalid = false;

	FTargetFileVisitor(const TSet<FString>& InOriginalTargetNames, TSet<FString>& InRemainingTargetNames, FDateTime InMaxDateTime)
		: OriginalTargetNames(InOriginalTargetNames)
		, RemainingTargetNames(InRemainingTargetNames)
		, MaxDateTime(InMaxDateTime)
	{
	}

	virtual bool Visit(const TCHAR* InFileNameOrDirectory, const FFileStatData& StatData) override
	{
		if (StatData.bIsDirectory)
		{
			SubDirectories.Add(InFileNameOrDirectory);
			return true;
		}
		
		FStringView FileNameOrDirectory(InFileNameOrDirectory);

		// NOTE: This code needs to behave the same as FindAllRulesSourceFiles in Rules.cs
		if (FStringView WithoutExtension = UE::String::RemoveFromEnd(FileNameOrDirectory, TEXTVIEW(".Target.cs"), ESearchCase::IgnoreCase);
			WithoutExtension.Len() != FileNameOrDirectory.Len())
		{
			FString TargetName = FPaths::GetCleanFilename(FString(WithoutExtension));

			// skip target rules that are platform extension or platform group specializations
			// Matches logic found in QueryTargetsMode.cs WriteTargetInfo
			FString Start, End;
			if (TargetName.Split(TEXT("_"), &Start, &End) && OriginalTargetNames.Contains(Start))
			{
				return true;
			}

			if (StatData.ModificationTime >= MaxDateTime)
			{	
				UE_LOGF(LogDesktopPlatform, Log, "Found target file %ls newer than cache", InFileNameOrDirectory);
				bCacheInvalid = true;
			}
			// Any target file found on disk not in the TargetInfo.json is assumed to not be 
			// relevant for the current build since anything new on disk hasn't been built at all. 
			if (RemainingTargetNames.Remove(TargetName) != 1)
			{
				UE_LOGF(LogDesktopPlatform, Log, "Found target file %ls not present in cache. Ignoring.", InFileNameOrDirectory);
				return true;
			}
			
			return !bCacheInvalid;
		}
		else if (FileNameOrDirectory.EndsWith(TEXTVIEW(".uplugin")))
		{
			bSearchSubDirectories = false;
			bSearchPluginForTargets = true;
			return true;
		}
		else if (FileNameOrDirectory.EndsWith(TEXTVIEW(".Build.cs"))
			|| FileNameOrDirectory.EndsWith(TEXTVIEW(".automation.csproj"))
		 	|| FileNameOrDirectory.EndsWith(TEXTVIEW(".ubtplugin.csproj"))
			|| FileNameOrDirectory.EndsWith(TEXTVIEW(".ubtignore"))
		)
		{
			bSearchSubDirectories = false;
			return true;
		}

		return true;
	}
};

// Note: This function must find all target files found by CreateProjectRulesAssembly in RulesCompiler.cs
bool IsTargetInfoValid(const TArray<FTargetInfo>& Targets, TArray<FString>& DirectoryNames, const FDateTime& LastModifiedTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("IsTargetInfoValid");
	if (FApp::GetEngineIsPromotedBuild())
	{
		// Promoted builds may not have source code, so we will assume all supplied targets are valid since they will not appear on disk
		return true;
	}
	
	// Create the state 
	TSet<FString> RemainingTargetNames;
	for (const FTargetInfo& Target : Targets)
	{
		RemainingTargetNames.Add(Target.Name);
	}

	TSet<FString> OriginalTargetNames = RemainingTargetNames;

	IFileManager& FM = IFileManager::Get();
	// Loop through all the directories
	for(int Idx = 0; Idx < DirectoryNames.Num(); Idx++)
	{
		// UE_LOGF(LogDesktopPlatform, Verbose, "Checking %ls for target files", *DirectoryNames[Idx]);
		FTargetFileVisitor Visitor(OriginalTargetNames, RemainingTargetNames, LastModifiedTime);
		FM.IterateDirectoryStat(*DirectoryNames[Idx], Visitor);
		if (Visitor.bCacheInvalid)
		{
			return false;
		}
		if (RemainingTargetNames.Num() == 0)
		{
			break; 
		}
		if (Visitor.bSearchPluginForTargets)
		{
			FString SourceDir = DirectoryNames[Idx] / TEXT("Source");
			if (FM.DirectoryExists(*SourceDir))
			{
				DirectoryNames.Emplace(MoveTemp(SourceDir));
			}
			FString TestsDir = DirectoryNames[Idx] / TEXT("Tests");
			if (FM.DirectoryExists(*TestsDir))
			{
				DirectoryNames.Emplace(MoveTemp(TestsDir));				
			}
		}
		if (Visitor.bSearchSubDirectories)
		{
			DirectoryNames.Append(MoveTemp(Visitor.SubDirectories));
		}
	}

	for (const FString& Target : RemainingTargetNames)
	{
		UE_LOGF(LogDesktopPlatform, Log, "Failed to find target file for %ls, cache out of date.", *Target);
	}
	// If we found all the previous target files
	return RemainingTargetNames.Num() == 0;
}

const TArray<FTargetInfo>& FDesktopPlatformBase::GetTargetsForProject(const FString& ProjectFile) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FDesktopPlatformBase::GetTargetsForProject");
	// Normalize the project filename
	FString NormalizedProjectFile = ProjectFile;
	FPaths::NormalizeFilename(NormalizedProjectFile);

	// Check if there's already a cached entry for this project
	const TArray<FTargetInfo>* Targets = ProjectFileToTargets.Find(NormalizedProjectFile);
	if (Targets != nullptr)
	{
		return *Targets;
	}

	// Get the base directory for the project (or the engine directory)
	FString ProjectDir;
	if(ProjectFile.Len() == 0)
	{
		ProjectDir = FPaths::EngineDir();
	}
	else
	{
		ProjectDir = FPaths::GetPath(ProjectFile);
	}

	FString ProjectSourceDir = ProjectDir / TEXT("Source");

	// Get the path to the info filename
	FString InfoFileName = ProjectDir / TEXT("Intermediate/TargetInfo.json");

	// Check if the file already exists
	FFileStatData StatData = IFileManager::Get().GetStatData(*InfoFileName);
	if(StatData.bIsValid)
	{
		// Read it in and check it's still valid
		TArray<FTargetInfo> NewTargets;
		if(ReadTargetInfo(InfoFileName, NewTargets))
		{
			TArray<FString> DirectoryNames = { 
				ProjectSourceDir,
				ProjectDir / TEXT("Plugins"),
				ProjectDir / TEXT("Platforms"), 
				ProjectDir / TEXT("Restricted"),
				};
			if(IsTargetInfoValid(NewTargets, DirectoryNames, StatData.ModificationTime))
			{
				return ProjectFileToTargets.Emplace(MoveTemp(NormalizedProjectFile), MoveTemp(NewTargets));
			}
		}
	}

	// Get the project source directory. If it doesn't exist, there are no targets for this project.
	if (!IFileManager::Get().DirectoryExists(*ProjectSourceDir))
	{
		return ProjectFileToTargets.Add(NormalizedProjectFile, TArray<FTargetInfo>());
	}

	// Otherwise, we'll have to run UBT to update it
	FString Arguments = TEXT("-Mode=QueryTargets");
	if(ProjectFile.Len() > 0)
	{
		Arguments += FString::Printf(TEXT(" -Project=\"%s\""), *IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProjectFile));
	}
	Arguments += FString::Printf(TEXT(" -Output=\"%s\""), *IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*InfoFileName));
	Arguments += TEXT(" -IncludeAllTargets");
	// we only want this game's targets, not every program under the sun (note if running UnrealEditor without a project, this will
	// still list all the programs). This matches what GenerateProjectFiles outputs when it performs this step
	Arguments += TEXT(" -DontIncludeParentAssembly");
	Arguments += TEXT(" -DontIncludeProgramTargets");

	// Run UBT to update the list of targets. Try to run it without building first.
	FString Output;
	int32 ReturnCode = 0;
	if (!FPaths::FileExists(GetUnrealBuildToolExecutableFilename(FPaths::RootDir())) || !const_cast<FDesktopPlatformBase*>(this)->InvokeUnrealBuildToolSync(Arguments, *GLog, true, ReturnCode, Output) || ReturnCode != 0)
	{
		// If that failed, try to build and run again. Build machines should always have an up-to-date copy, and shouldn't build anything without being told to do so.
		if (!GIsBuildMachine)
		{
			const_cast<FDesktopPlatformBase*>(this)->InvokeUnrealBuildToolSync(Arguments, *GLog, false, ReturnCode, Output);
		}
	}

	// Try to read the new targets
	TArray<FTargetInfo> NewTargets;
	if(!ReadTargetInfo(InfoFileName, NewTargets))
	{
		UE_LOGF(LogDesktopPlatform, Warning, "Unable to read target info for %ls", (ProjectFile.Len() == 0)? TEXT("engine") : *ProjectFile);
	}

	// Add it to the cache
	return ProjectFileToTargets.Emplace(MoveTemp(NormalizedProjectFile), MoveTemp(NewTargets));
}

const TArray<FTargetInfo>& FDesktopPlatformBase::GetTargetsForCurrentProject() const
{
	return GetTargetsForProject(FPaths::GetProjectFilePath());
}

FString FDesktopPlatformBase::GetDefaultProjectCreationPath()
{
	// My Documents
	const FString DefaultProjectSubFolder = TEXT("Unreal Projects");
	return FString(FPlatformProcess::UserDir()) + DefaultProjectSubFolder;
}

void FDesktopPlatformBase::ReadLauncherInstallationList()
{
	FString InstalledListFile = FString(FPlatformProcess::ApplicationSettingsDir()) / TEXT("UnrealEngineLauncher/LauncherInstalled.dat");

	// If the file does not exist, manually check for the 4.0 or 4.1 manifest
	FDateTime NewListTimestamp = IFileManager::Get().GetTimeStamp(*InstalledListFile);
	if(NewListTimestamp == FDateTime::MinValue())
	{
		if(LauncherInstallationList.Num() == 0)
		{
			CheckForLauncherEngineInstallation(TEXT("40003"), TEXT("UE_4.0"), LauncherInstallationList);
			CheckForLauncherEngineInstallation(TEXT("1040003"), TEXT("UE_4.1"), LauncherInstallationList);
		}
	}
	else if(NewListTimestamp != LauncherInstallationTimestamp)
	{
		// Read the installation manifest
		FString InstalledText;
		if (FFileHelper::LoadFileToString(InstalledText, *InstalledListFile))
		{
			// Deserialize the object
			TSharedPtr< FJsonObject > RootObject;
			TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(InstalledText);
			if (FJsonSerializer::Deserialize(Reader, RootObject) && RootObject.IsValid())
			{
				// Parse the list of installations
				TArray< TSharedPtr<FJsonValue> > InstallationList = RootObject->GetArrayField(TEXT("InstallationList"));
				for(int32 Idx = 0; Idx < InstallationList.Num(); Idx++)
				{
					TSharedPtr<FJsonObject> InstallationItem = InstallationList[Idx]->AsObject();

					FString AppName = InstallationItem->GetStringField(TEXT("AppName"));
					FString InstallLocation = InstallationItem->GetStringField(TEXT("InstallLocation"));
					if(AppName.Len() > 0 && InstallLocation.Len() > 0)
					{
						FPaths::NormalizeDirectoryName(InstallLocation);
						LauncherInstallationList.Add(AppName, InstallLocation);
					}
				}
			}
			LauncherInstallationTimestamp = NewListTimestamp;
		}
	}
}

void FDesktopPlatformBase::CheckForLauncherEngineInstallation(const FString &AppId, const FString &Identifier, TMap<FString, FString> &OutInstallations)
{
	FString ManifestText;
	FString ManifestFileName = FString(FPlatformProcess::ApplicationSettingsDir()) / FString::Printf(TEXT("UnrealEngineLauncher/Data/Manifests/%s.manifest"), *AppId);

	if (FFileHelper::LoadFileToString(ManifestText, *ManifestFileName))
	{
		TSharedPtr< FJsonObject > RootObject;
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(ManifestText);
		if (FJsonSerializer::Deserialize(Reader, RootObject) && RootObject.IsValid())
		{
			TSharedPtr<FJsonObject> CustomFieldsObject = RootObject->GetObjectField(TEXT("CustomFields"));
			if (CustomFieldsObject.IsValid())
			{
				FString InstallLocation = CustomFieldsObject->GetStringField(TEXT("InstallLocation"));
				if (InstallLocation.Len() > 0)
				{
					OutInstallations.Add(Identifier, InstallLocation);
				}
			}
		}
	}
}

int32 FDesktopPlatformBase::ParseReleaseVersion(const FString& Version)
{
	TCHAR *End;

	uint64 Major = FCString::Strtoui64(*Version, &End, 10);
	if (Major >= MAX_int16 || *(End++) != '.')
	{
		return INDEX_NONE;
	}

	uint64 Minor = FCString::Strtoui64(End, &End, 10);
	if (Minor >= MAX_int16 || *End != 0)
	{
		return INDEX_NONE;
	}

	return IntCastChecked<int32>((Major << 16) + Minor);
}

TSharedPtr<FJsonObject> FDesktopPlatformBase::LoadProjectFile(const FString &FileName)
{
	FString FileContents;

	if (!FFileHelper::LoadFileToString(FileContents, *FileName))
	{
		return TSharedPtr<FJsonObject>(NULL);
	}

	TSharedPtr< FJsonObject > JsonObject;
	TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(FileContents);
	if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
	{
		return TSharedPtr<FJsonObject>(NULL);
	}

	return JsonObject;
}

bool FDesktopPlatformBase::SaveProjectFile(const FString &FileName, TSharedPtr<FJsonObject> Object)
{
	FString FileContents;

	TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create(&FileContents);
	if (!FJsonSerializer::Serialize(Object.ToSharedRef(), Writer))
	{
		return false;
	}

	if (!FFileHelper::SaveStringToFile(FileContents, *FileName))
	{
		return false;
	}

	return true;
}

const FUProjectDictionary &FDesktopPlatformBase::GetCachedProjectDictionary(const FString& RootDir)
{
	FString NormalizedRootDir = RootDir;
	FPaths::NormalizeDirectoryName(NormalizedRootDir);

	FUProjectDictionary *Dictionary = CachedProjectDictionaries.Find(NormalizedRootDir);
	if(Dictionary == NULL)
	{
		Dictionary = &CachedProjectDictionaries.Add(RootDir, FUProjectDictionary(RootDir));
	}
	return *Dictionary;
}

void FDesktopPlatformBase::GetProjectBuildProducts(const FString& ProjectDir, TArray<FString> &OutFileNames, TArray<FString> &OutDirectoryNames)
{
	FString NormalizedProjectDir = ProjectDir;
	FPaths::NormalizeDirectoryName(NormalizedProjectDir);

	// Find all the build roots
	TArray<FString> BuildRootDirectories;
	BuildRootDirectories.Add(NormalizedProjectDir);

	// Add all the plugin directories
	TArray<FString> PluginFileNames;
	IFileManager::Get().FindFilesRecursive(PluginFileNames, *(NormalizedProjectDir / TEXT("Plugins")), TEXT("*.uplugin"), true, false);
	for(int32 Idx = 0; Idx < PluginFileNames.Num(); Idx++)
	{
		BuildRootDirectories.Add(FPaths::GetPath(PluginFileNames[Idx]));
	}

	// Add all the intermediate directories
	for(int32 Idx = 0; Idx < BuildRootDirectories.Num(); Idx++)
	{
		OutDirectoryNames.Add(BuildRootDirectories[Idx] / TEXT("Intermediate"));
	}

	// Add the files in the cleaned directories to the output list
	for(int32 Idx = 0; Idx < OutDirectoryNames.Num(); Idx++)
	{
		IFileManager::Get().FindFilesRecursive(OutFileNames, *OutDirectoryNames[Idx], TEXT("*"), true, false, false);
	}
}

FString FDesktopPlatformBase::GetEngineSavedConfigDirectory(const FString& Identifier)
{
	// Get the game agnostic config dir
	const FString UserDir = GetUserDir(Identifier);
	if (!UserDir.IsEmpty())
	{
		return UserDir / TEXT("Saved/Config") / ANSI_TO_TCHAR(FPlatformProperties::PlatformName());
	}

	return FString();
}

bool FDesktopPlatformBase::EnumerateProjectsKnownByEngine(const FString &Identifier, bool bIncludeNativeProjects, TArray<FString> &OutProjectFileNames)
{
	// Get the engine root directory
	FString RootDir;
	if (!GetEngineRootDirFromIdentifier(Identifier, RootDir))
	{
		return false;
	}

	FString GameAgnosticConfigDir = GetEngineSavedConfigDirectory(Identifier);
	if (GameAgnosticConfigDir.Len() == 0)
	{
		return false;
	}

	// Find all the created project directories. Start with the default project creation path.
	TArray<FString> SearchDirectories;
	SearchDirectories.AddUnique(GetDefaultProjectCreationPath());

	UE_LOGF(LogDesktopPlatform, Log, "Enumerating Projects From Engine Ver: %ls", *Identifier);

	UE_LOGF(LogDesktopPlatform, Log, "Looking for directories to scan from : %ls", *GameAgnosticConfigDir);
	// Load the config file
	FConfigFile GameAgnosticConfig;
	if (!FConfigCacheIni::LoadExternalIniFile(GameAgnosticConfig, TEXT("EditorSettings"), NULL, *GameAgnosticConfigDir, false))
	{
		FString PreviousConfigDir = MoveTemp(GameAgnosticConfigDir);

		// Load from the legacy path. Most likely a pre-UE5 engine install
		GameAgnosticConfigDir = GetLegacyEngineSavedConfigDirectory(Identifier);

		UE_LOGF(LogDesktopPlatform, Log, "%ls not found, looking for directories in %ls", *PreviousConfigDir , *GameAgnosticConfigDir);

		FConfigCacheIni::LoadExternalIniFile(GameAgnosticConfig, TEXT("EditorSettings"), NULL, *GameAgnosticConfigDir, false);
	}

	// Find the editor game-agnostic settings
	const FConfigSection* Section = GameAgnosticConfig.FindSection(TEXT("/Script/UnrealEd.EditorSettings"));

	if (Section == NULL)
	{
		FConfigCacheIni::LoadExternalIniFile(GameAgnosticConfig, TEXT("EditorGameAgnostic"), NULL, *GameAgnosticConfigDir, false);
		Section = GameAgnosticConfig.FindSection(TEXT("/Script/UnrealEd.EditorGameAgnosticSettings"));
	}

	if (GameAgnosticConfig.IsEmpty())
	{
		UE_LOGF(LogDesktopPlatform, Log, "Config not found");
	}

	if(Section != NULL)
	{
		UE_LOGF(LogDesktopPlatform, Log, "Searching for previously created project directories...");

		// Add in every path that the user has ever created a project file. This is to catch new projects showing up in the user's project folders
		TArray<FString> AdditionalDirectories;
		Section->MultiFind(TEXT("CreatedProjectPaths"), AdditionalDirectories);
		for(int Idx = 0; Idx < AdditionalDirectories.Num(); Idx++)
		{
			FPaths::NormalizeDirectoryName(AdditionalDirectories[Idx]);

			UE_LOGF(LogDesktopPlatform, Log, "Found directory: \"%ls\"", *AdditionalDirectories[Idx]);
			SearchDirectories.AddUnique(AdditionalDirectories[Idx]);
		}

		// Also add in all the recently opened projects
		TArray<FString> RecentlyOpenedFiles;
		Section->MultiFind(TEXT("RecentlyOpenedProjectFiles"), RecentlyOpenedFiles);
		for(int Idx = 0; Idx < RecentlyOpenedFiles.Num(); Idx++)
		{
			FPaths::NormalizeFilename(RecentlyOpenedFiles[Idx]);

			UE_LOGF(LogDesktopPlatform, Log, "Found project \"%ls\" in recently opened files", *RecentlyOpenedFiles[Idx]);

			OutProjectFileNames.AddUnique(RecentlyOpenedFiles[Idx]);
		}		
	}

	// Find all the other projects that are in the search directories
	for(int Idx = 0; Idx < SearchDirectories.Num(); Idx++)
	{
		TArray<FString> ProjectFolders;
		IFileManager::Get().FindFiles(ProjectFolders, *(SearchDirectories[Idx] / TEXT("*")), false, true);

		for(int32 FolderIdx = 0; FolderIdx < ProjectFolders.Num(); FolderIdx++)
		{
			TArray<FString> ProjectFiles;
			IFileManager::Get().FindFiles(ProjectFiles, *(SearchDirectories[Idx] / ProjectFolders[FolderIdx] / TEXT("*.uproject")), true, false);

			for(int32 FileIdx = 0; FileIdx < ProjectFiles.Num(); FileIdx++)
			{
				FString ProjName = SearchDirectories[Idx] / ProjectFolders[FolderIdx] / ProjectFiles[FileIdx];

				UE_LOGF(LogDesktopPlatform, Log, "Found project \"%ls\" in previously created project directory", *ProjName);

				OutProjectFileNames.AddUnique(MoveTemp(ProjName));
			}
		}
	}

	UE_LOGF(LogDesktopPlatform, Log, "Searching for projects in .uprojectdirs");

	// Find all the native projects, and either add or remove them from the list depending on whether we want native projects
	const FUProjectDictionary &Dictionary = GetCachedProjectDictionary(RootDir);
	if(bIncludeNativeProjects)
	{
		TArray<FString> NativeProjectPaths = Dictionary.GetProjectPaths();
		for(int Idx = 0; Idx < NativeProjectPaths.Num(); Idx++)
		{
			if(!NativeProjectPaths[Idx].Contains(TEXT("/Templates/")))
			{
				UE_LOGF(LogDesktopPlatform, Log, "Found project \"%ls\" in .uprojectdirs", *NativeProjectPaths[Idx]);

				OutProjectFileNames.AddUnique(NativeProjectPaths[Idx]);
			}
		}
	}
	else
	{
		TArray<FString> NativeProjectPaths = Dictionary.GetProjectPaths();
		for(int Idx = 0; Idx < NativeProjectPaths.Num(); Idx++)
		{
			OutProjectFileNames.Remove(NativeProjectPaths[Idx]);
		}
	}

	return true;
}

bool FDesktopPlatformBase::BuildUnrealBuildTool(const FString& RootDir, FOutputDevice& Ar)
{
#if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX
	Ar.Logf(TEXT("Building UnrealBuildTool in %s..."), *RootDir);

	// Check the project file exists
	FString CsProjLocation = GetUnrealBuildToolProjectFileName(RootDir);
	if (!FPaths::FileExists(CsProjLocation))
	{
		Ar.Logf(TEXT("Project file not found at %s"), *CsProjLocation);
		return false;
	}

	FString CompilerExecutableFilename;
	FString CmdLineParams;

	if constexpr (PLATFORM_WINDOWS)
	{
		FString ScriptPath = FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Build/BatchFiles/BuildUBT.bat"));
		CompilerExecutableFilename = TEXT("cmd.exe");
		CmdLineParams = FString::Printf(TEXT("/c \"%s\""), *ScriptPath);
	}
	else if constexpr (PLATFORM_MAC)
	{
		FString ScriptPath = FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Build/BatchFiles/BuildUBT.sh"));
		CompilerExecutableFilename = TEXT("/bin/sh");
		CmdLineParams = FString::Printf(TEXT("\"%s\""), *ScriptPath);
	}
	else if constexpr (PLATFORM_LINUX)
	{
		FString ScriptPath = FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Build/BatchFiles/BuildUBT.sh"));
		CompilerExecutableFilename = TEXT("/bin/bash");
		CmdLineParams = FString::Printf(TEXT("\"%s\""), *ScriptPath);
	}

	// Spawn the compiler
	Ar.Logf(TEXT("Running: %s %s"), *CompilerExecutableFilename, *CmdLineParams);
	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = bLaunchHidden;

	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(*CompilerExecutableFilename, *CmdLineParams, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, NULL, 0, NULL, PipeWrite);
	if (!ProcHandle.IsValid())
	{
		Ar.Log(TEXT("Failed to start process."));
		FPlatformProcess::ClosePipe(PipeRead, PipeWrite);
		return false;
	}
	FPlatformProcess::WaitForProc(ProcHandle);
	FPlatformProcess::CloseProc(ProcHandle);
	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);

	// If the executable appeared where we expect it, then we were successful
	FString UnrealBuildToolExePath = GetUnrealBuildToolExecutableFilename(RootDir);
	if (!FPaths::FileExists(UnrealBuildToolExePath))
	{
		Ar.Logf(TEXT("Missing %s after build"), *UnrealBuildToolExePath);
		return false;
	}

	return true;
#else // #if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX
	Ar.Log(TEXT("Unsupported platform, unable to build UnrealBuildTool."));
	return false;
#endif // #else // #if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX
}

bool FDesktopPlatformBase::ReadTargetInfo(const FString& FileName, TArray<FTargetInfo>& Targets)
{
	// Read the file to a string
	FString Contents;
	if (!FFileHelper::LoadFileToString(Contents, *FileName))
	{
		return false;
	}

	// Deserialize a JSON object from the string
	TSharedPtr<FJsonObject> ObjectPtr;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Contents);
	if (!FJsonSerializer::Deserialize(Reader, ObjectPtr) || !ObjectPtr.IsValid())
	{
		return false;
	}

	// Get the targets array
	const TArray<TSharedPtr<FJsonValue>>* TargetArray;
	if (!ObjectPtr->TryGetArrayField(TEXT("Targets"), TargetArray))
	{
		return false;
	}

	// Presize the output array
	Targets.SetNum(TargetArray->Num());

	// Parse the entries
	FString BaseDir = FPaths::GetPath(FileName);
	for (int Idx = 0; Idx < TargetArray->Num(); Idx++)
	{
		const FJsonValue& TargetValue = *(*TargetArray)[Idx].Get();
		if (TargetValue.Type != EJson::Object)
		{
			return false;
		}

		const FJsonObject& TargetObject = *TargetValue.AsObject();
		if(!TargetObject.TryGetStringField(TEXT("Name"), Targets[Idx].Name) || !TargetObject.TryGetStringField(TEXT("Path"), Targets[Idx].Path))
		{
			return false;
		}

		FString Type;
		if (!TargetObject.TryGetStringField(TEXT("Type"), Type))
		{
			return false;
		}
		if(!LexTryParseString(Targets[Idx].Type, *Type) || Targets[Idx].Type == EBuildTargetType::Unknown)
		{
			return false;
		}
		
		if (bool bDefaultTarget; TargetObject.TryGetBoolField(TEXT("DefaultTarget"), bDefaultTarget))
		{
			Targets[Idx].DefaultTarget = bDefaultTarget;
		}

		FString Path = FPaths::ConvertRelativePathToFull(FPaths::Combine(BaseDir, Targets[Idx].Path));
		FPaths::MakePlatformFilename(Path);

		Targets[Idx].Path = Path;
	}

	return true;
}

FString FDesktopPlatformBase::GetUserDir(const FString& Identifier)
{
	// Get the engine root directory
	FString RootDir;
	if (!GetEngineRootDirFromIdentifier(Identifier, RootDir))
	{
		return FString();
	}

	// Get the path to the game agnostic settings
	FString UserDir;
	if (IsStockEngineRelease(Identifier))
	{
		UserDir = FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), *Identifier);
	}
	else
	{
		UserDir = FPaths::Combine(*RootDir, TEXT("Engine"));
	}

	return UserDir;
}

FString FDesktopPlatformBase::GetLegacyEngineSavedConfigDirectory(const FString& Identifier)
{
	const FString UserDir = GetUserDir(Identifier);
	if (!UserDir.IsEmpty())
	{
		return UserDir / TEXT("Saved/Config") / ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName());
	}

	return FString();
}

FString FDesktopPlatformBase::GetUnrealBuildToolProjectFileName(const FString& RootDir) const
{
	return FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Source/Programs/UnrealBuildTool/UnrealBuildTool.csproj"));
}

FString FDesktopPlatformBase::GetUnrealBuildToolExecutableFilename(const FString& RootDir) const
{
	FConfigFile Config;
	if (FConfigCacheIni::LoadExternalIniFile(Config, TEXT("Engine"), *FPaths::Combine(RootDir, TEXT("Engine/Config/")), *FPaths::Combine(RootDir, TEXT("Engine/Config/")), true, NULL, false, /*bWriteDestIni*/ false))
	{
		FString Entry;
		if( Config.GetString( TEXT("PlatformPaths"), TEXT("UnrealBuildTool"), Entry ))
		{
			FString NewPath = FPaths::ConvertRelativePathToFull(RootDir / Entry);
			return NewPath;
		}
	}
	

	return FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Binaries/DotNET/UnrealBuildTool/UnrealBuildTool.exe"));
}

#undef LOCTEXT_NAMESPACE
