// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"
#include "IDesktopPlatform.h"
#include "Misc/UProjectInfo.h"
#include "Misc/CompilationResult.h"

class FEngineVersion;
class FJsonObject;

PRAGMA_DISABLE_DEPRECATION_WARNINGS // For FDesktopPlatformBase::GetSolutionPath() implementation

class FDesktopPlatformBase : public IDesktopPlatform
{
public:
	FDesktopPlatformBase();

	// IDesktopPlatform Implementation
	virtual FString GetEngineDescription(const FString& Identifier) override;
	virtual FString GetCurrentEngineIdentifier() override;

	virtual void EnumerateLauncherEngineInstallations(TMap<FString, FString> &OutInstallations) override;
	virtual void EnumerateLauncherSampleInstallations(TArray<FString> &OutInstallations) override;
	virtual void EnumerateLauncherSampleProjects(TArray<FString> &OutFileNames) override;
	virtual bool GetEngineRootDirFromIdentifier(const FString &Identifier, FString &OutRootDir) override;
	virtual bool GetEngineIdentifierFromRootDir(const FString &RootDir, FString &OutIdentifier) override;

	virtual bool GetDefaultEngineIdentifier(FString &OutIdentifier) override;
	virtual bool GetDefaultEngineRootDir(FString &OutRootDir) override;
	virtual FString GetEngineSavedConfigDirectory(const FString& Identifier) override;
	
	virtual bool IsPreferredEngineIdentifier(const FString &Identifier, const FString &OtherIdentifier) override;

	virtual bool TryGetEngineVersion(const FString& RootDir, FEngineVersion& OutVersion) override;
	virtual bool IsStockEngineRelease(const FString &Identifier) override;
	virtual bool TryParseStockEngineVersion(const FString& Identifier, FEngineVersion& OutVersion) override;
	virtual bool IsSourceDistribution(const FString &RootDir) override;
	virtual bool IsPerforceBuild(const FString &RootDir) override;
	virtual bool IsValidRootDirectory(const FString &RootDir) override;

	virtual bool SetEngineIdentifierForProject(const FString &ProjectFileName, const FString &Identifier) override;
	virtual bool GetEngineIdentifierForProject(const FString &ProjectFileName, FString &OutIdentifier) override;

	virtual bool OpenProject(const FString& ProjectFileName) override;

	virtual bool CleanGameProject(const FString& ProjectDir, FString& OutFailPath, FFeedbackContext* Warn) override;
	virtual bool CompileGameProject(const FString& RootDir, const FString& ProjectFileName, FFeedbackContext* Warn, ECompilationResult::Type* OutResult) override;
	virtual bool GenerateProjectFiles(const FString& RootDir, const FString& ProjectFileName, FFeedbackContext* Warn, FString LogFilePath = FString()) override;
	virtual bool IsUnrealBuildToolAvailable() override;
	virtual bool InvokeUnrealBuildToolSync(const FString& InCmdLineParams, FOutputDevice &Ar, bool bSkipBuildUBT, int32& OutReturnCode, FString& OutProcOutput) override;
	virtual FProcHandle InvokeUnrealBuildToolAsync(const FString& InCmdLineParams, FOutputDevice &Ar, void*& OutReadPipe, void*& OutWritePipe, bool bSkipBuildUBT = false) override;
	virtual bool RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn) override final;
	virtual bool IsUnrealBuildToolRunning() override;

    virtual bool GetOidcAccessToken(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) override;
	virtual bool GetOidcAccessTokenWithRemoteConfig(const FString& RootDir, const FString& RemoteHost, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) override;
    virtual bool GetOidcAccessToken(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) override;
	virtual bool GetOidcAccessTokenWithRemoteConfig(const FString& RootDir, const FString& RemoteHost, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) override;
	virtual bool GetOidcTokenStatus(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, FFeedbackContext* Warn, int& OutStatus) override;

	virtual bool GetHordeUrl(FString& OutHordeUrl, FString* OutHordeUrlConfigSource = nullptr) override;
	virtual void SetHordeUrl(const FString& HordeUrl) override;
	virtual bool GetHordeAccessToken(const FString& HordeUrl, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) override;
	virtual bool GetHordeAccessToken(const FString& HordeUrl, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) override;

	virtual const TArray<FTargetInfo>& GetTargetsForProject(const FString& ProjectFile) const override;
	virtual const TArray<FTargetInfo>& GetTargetsForCurrentProject() const override;

	virtual bool EnumerateProjectsKnownByEngine(const FString &Identifier, bool bIncludeNativeProjects, TArray<FString> &OutProjectFileNames) override;
	virtual FString GetDefaultProjectCreationPath() override;

private:
	FString CurrentEngineIdentifier;
	FDateTime LauncherInstallationTimestamp;
	TMap<FString, FString> LauncherInstallationList;
	TMap<FString, FUProjectDictionary> CachedProjectDictionaries;
	mutable TMap<FString, TArray<FTargetInfo>> ProjectFileToTargets;
	bool bInitializedHordeServerUrl = false;
	FString HordeServerUrl;
	FString HordeServerUrlConfigSource;

	static FString ReadHordeUrlWithoutCache(FString& OutHordeUrlConfigSource);

	void ReadLauncherInstallationList();
	void CheckForLauncherEngineInstallation(const FString &AppId, const FString &Identifier, TMap<FString, FString> &OutInstallations);
	int32 ParseReleaseVersion(const FString &Version);

	TSharedPtr<FJsonObject> LoadProjectFile(const FString &FileName);
	bool SaveProjectFile(const FString &FileName, TSharedPtr<FJsonObject> Object);

	const FUProjectDictionary &GetCachedProjectDictionary(const FString& RootDir);

	void GetProjectBuildProducts(const FString& ProjectFileName, TArray<FString> &OutFileNames, TArray<FString> &OutDirectoryNames);
	bool BuildUnrealBuildTool(const FString& RootDir, FOutputDevice &Ar);

	bool InvokeOidcTokenToolSync(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutReturnCode, FString& OutProcOutput);
	FProcHandle InvokeOidcTokenToolAsync(const FString& InArguments, void*& OutReadPipe, void*& OutWritePipe);
	bool GetOidcAccessTokenInternal(const FString& RootDir, const FString& Arguments, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin);
	bool GetOidcAccessTokenInteractive(const FString& RootDir, const FString& Arguments, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, int32& OutReturnCode);

	static bool ReadTargetInfo(const FString& FileName, TArray<FTargetInfo>& Targets);

	FString GetUserDir(const FString& Identifier);

	/** Pre-UE5 location of saved configs */
	FString GetLegacyEngineSavedConfigDirectory(const FString& Identifier);
protected:

	FString GetUnrealBuildToolProjectFileName(const FString& RootDir) const;
	FString GetUnrealBuildToolExecutableFilename(const FString& RootDir) const;	
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS
