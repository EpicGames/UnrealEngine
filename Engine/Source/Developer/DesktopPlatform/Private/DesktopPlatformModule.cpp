// Copyright Epic Games, Inc. All Rights Reserved.

#include "DesktopPlatformModule.h"
#include "DesktopPlatformPrivate.h"
#include "Null/NullPlatformApplicationMisc.h"

IMPLEMENT_MODULE( FDesktopPlatformModule, DesktopPlatform );
DEFINE_LOG_CATEGORY(LogDesktopPlatform);

void FDesktopPlatformModule::StartupModule()
{
	if(FNullPlatformApplicationMisc::IsUsingNullApplication())
	{
		DesktopPlatform = new FDesktopPlatformNull();
	}
	else
	{
		DesktopPlatform = new FDesktopPlatform();
	}

	FPlatformMisc::SetEnvironmentVar(TEXT("UE_DesktopUnrealProcess"), TEXT("1"));
}

void FDesktopPlatformModule::ShutdownModule()
{
	FPlatformMisc::SetEnvironmentVar(TEXT("UE_DesktopUnrealProcess"), TEXT("0"));

	if (DesktopPlatform != NULL)
	{
		delete DesktopPlatform;
		DesktopPlatform = NULL;
	}
}
