// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// IWYU pragma: begin_exports
#if PLATFORM_WINDOWS
#include "Windows/DesktopPlatformWindows.h"
#elif PLATFORM_MAC
#include "Mac/DesktopPlatformMac.h"
#elif PLATFORM_LINUX
#include "Linux/DesktopPlatformLinux.h"
#else
#include "DesktopPlatformStub.h"
#endif

#include "Null/DesktopPlatformNull.h"
// IWYU pragma: end_exports

DECLARE_LOG_CATEGORY_EXTERN(LogDesktopPlatform, Log, All);
