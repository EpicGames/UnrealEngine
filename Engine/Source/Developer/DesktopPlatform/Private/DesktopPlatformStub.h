// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DesktopPlatformBase.h"
#include "Misc/Paths.h"

class FDesktopPlatformStub : public FDesktopPlatformBase
{
public:
	// IDesktopPlatform Implementation
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames) override { return false; }
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex) override { return false; }
	virtual bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames) override { return false; }
	virtual bool OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, FString& OutFolderName) override { return false; }
	virtual bool OpenFontDialog(const void* ParentWindowHandle, FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags) override { return false; }

	virtual bool RegisterEngineInstallation(const FString &RootDir, FString &OutIdentifier) override { return false; }
	virtual void EnumerateEngineInstallations(TMap<FString, FString> &OutInstallations) override {}

	virtual bool VerifyFileAssociations() override { return false; }
	virtual bool UpdateFileAssociations() override { return false; }

	using FDesktopPlatformBase::RunUnrealBuildTool;
	virtual bool RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutExitCode) override { OutExitCode = 1;  return false; }
	virtual bool IsUnrealBuildToolRunning() override { return false; }

	virtual FString GetOidcTokenExecutableFilename(const FString& RootDir) const override { return ""; }

	virtual FFeedbackContext* GetNativeFeedbackContext() override { return nullptr; }

	virtual FString GetUserTempPath() override { return FPaths::ProjectIntermediateDir() + TEXT("UserTemp/"); }
};

typedef FDesktopPlatformStub FDesktopPlatform;
