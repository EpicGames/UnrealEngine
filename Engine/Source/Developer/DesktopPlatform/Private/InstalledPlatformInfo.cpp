// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstalledPlatformInfo.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "PlatformInfo.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"

#define LOCTEXT_NAMESPACE "InstalledPlatformInfo"

DEFINE_LOG_CATEGORY_STATIC(LogInstalledPlatforms, Log, All);

EProjectType EProjectTypeFromString(const FString& ProjectTypeName)
{
	if (FCString::Strcmp(*ProjectTypeName, TEXT("Any")) == 0)
	{
		return EProjectType::Any;
	}
	else if (FCString::Strcmp(*ProjectTypeName, TEXT("Code")) == 0)
	{
		return EProjectType::Code;
	}
	else if (FCString::Strcmp(*ProjectTypeName, TEXT("Content")) == 0)
	{
		return EProjectType::Content;
	}
	else
	{
		return EProjectType::Unknown;
	}
}

FInstalledPlatformInfo::FInstalledPlatformInfo()
{
	TArray<FString> InstalledPlatforms;
	GConfig->GetArray(TEXT("InstalledPlatforms"), TEXT("InstalledPlatformConfigurations"), InstalledPlatforms, GEngineIni);

	for (FString& InstalledPlatform : InstalledPlatforms)
	{
		ParsePlatformConfiguration(InstalledPlatform);
	}
}

void FInstalledPlatformInfo::ParsePlatformConfiguration(FString PlatformConfiguration)
{
	// Trim whitespace at the beginning.
	PlatformConfiguration.TrimStartInline();
	// Remove brackets.
	PlatformConfiguration.RemoveFromStart(TEXT("("));
	PlatformConfiguration.RemoveFromEnd(TEXT(")"));

	bool bCanCreateEntry = true;

	FString ConfigurationName;
	EBuildConfiguration Configuration = EBuildConfiguration::Unknown;
	if (!FParse::Value(*PlatformConfiguration, TEXT("Configuration="), ConfigurationName) || !LexTryParseString(Configuration, *ConfigurationName) || Configuration == EBuildConfiguration::Unknown)
	{
		UE_LOGF(LogInstalledPlatforms, Warning, "Unable to read configuration from %ls", *PlatformConfiguration);
		bCanCreateEntry = false;
	}

	FString	PlatformName;
	if (!FParse::Value(*PlatformConfiguration, TEXT("PlatformName="), PlatformName))
	{
		UE_LOGF(LogInstalledPlatforms, Warning, "Unable to read platform from %ls", *PlatformConfiguration);
		bCanCreateEntry = false;
	}

	FString PlatformTypeName;
	EBuildTargetType PlatformType;
	if (!FParse::Value(*PlatformConfiguration, TEXT("PlatformType="), PlatformTypeName) || !LexTryParseString(PlatformType, *PlatformTypeName))
	{
		PlatformType = EBuildTargetType::Game;
	}

	FString Architecture;
	FParse::Value(*PlatformConfiguration, TEXT("Architecture="), Architecture);

	FString RequiredFile;
	if (FParse::Value(*PlatformConfiguration, TEXT("RequiredFile="), RequiredFile))
	{
		RequiredFile = FPaths::Combine(*FPaths::RootDir(), *RequiredFile);
	}

	FString ProjectTypeName;
	EProjectType ProjectType =  EProjectType::Any;
	if (FParse::Value(*PlatformConfiguration, TEXT("ProjectType="), ProjectTypeName))
	{
		ProjectType = EProjectTypeFromString(ProjectTypeName);
	}
	if (ProjectType == EProjectType::Unknown)
	{
		UE_LOGF(LogInstalledPlatforms, Warning, "Unable to read project type from %ls", *PlatformConfiguration);
		bCanCreateEntry = false;
	}

	bool bCanBeDisplayed = false;
	FParse::Bool(*PlatformConfiguration, TEXT("bCanBeDisplayed="), bCanBeDisplayed);
	
	if (bCanCreateEntry)
	{
		FInstalledPlatformConfiguration NewConfig = {Configuration, PlatformName, PlatformType, Architecture, RequiredFile, ProjectType, bCanBeDisplayed};
		InstalledPlatformConfigurations.Add(NewConfig);
	}
}

FInstalledPlatformInfo& FInstalledPlatformInfo::Get()
{
	static FInstalledPlatformInfo InfoSingleton;
	return InfoSingleton;
}

bool FInstalledPlatformInfo::IsValidConfiguration(const EBuildConfiguration Configuration, EProjectType ProjectType) const
{
	return ContainsValidConfiguration(
		[Configuration, ProjectType](const FInstalledPlatformConfiguration& CurConfig)
		{
			return CurConfig.Configuration == Configuration
				&& (ProjectType == EProjectType::Any || CurConfig.ProjectType == EProjectType::Any
					|| CurConfig.ProjectType == ProjectType);
		}
	);
}

bool FInstalledPlatformInfo::IsValidPlatform(const FString& PlatformName, EProjectType ProjectType) const
{
	return ContainsValidConfiguration(
		[PlatformName, ProjectType](const FInstalledPlatformConfiguration& CurConfig)
		{
			return CurConfig.PlatformName == PlatformName
				&& (ProjectType == EProjectType::Any || CurConfig.ProjectType == EProjectType::Any
					|| CurConfig.ProjectType == ProjectType);
		}
	);
}

bool FInstalledPlatformInfo::IsValidPlatformAndConfiguration(const EBuildConfiguration Configuration, const FString& PlatformName, EProjectType ProjectType) const
{
	return ContainsValidConfiguration(
		[Configuration, PlatformName, ProjectType](const FInstalledPlatformConfiguration& CurConfig)
		{
			return CurConfig.Configuration == Configuration
				&& CurConfig.PlatformName == PlatformName
				&& (ProjectType == EProjectType::Any || CurConfig.ProjectType == EProjectType::Any
					|| CurConfig.ProjectType == ProjectType);
		}
	);
}

bool FInstalledPlatformInfo::CanDisplayPlatform(const FString& PlatformName, EProjectType ProjectType) const
{
	return ContainsMatchingConfiguration(
		[PlatformName, ProjectType](const FInstalledPlatformConfiguration& CurConfig)
		{
			return CurConfig.PlatformName == PlatformName && (CurConfig.bCanBeDisplayed
				|| ProjectType == EProjectType::Any || CurConfig.ProjectType == EProjectType::Any
				|| CurConfig.ProjectType == ProjectType);
		}
	);
}

bool FInstalledPlatformInfo::IsValidTargetType(EBuildTargetType TargetType) const
{
	return ContainsValidConfiguration(
		[TargetType](const FInstalledPlatformConfiguration& CurConfig)
		{
			return CurConfig.PlatformType == TargetType;
		}
	);
}

bool FInstalledPlatformInfo::IsValid(TOptional<EBuildTargetType> TargetType, TOptional<FString> Platform, TOptional<EBuildConfiguration> Configuration, EProjectType ProjectType, EInstalledPlatformState State) const
{
	if (!FApp::IsEngineInstalled())
	{
		return true;
	}

	for (const FInstalledPlatformConfiguration& Config : InstalledPlatformConfigurations)
	{
		// Check whether this configuration matches all the criteria
		if(TargetType.IsSet() && Config.PlatformType != TargetType.GetValue())
		{
			continue;
		}
		if(Platform.IsSet() && Config.PlatformName != Platform.GetValue())
		{
			continue;
		}
		if(Configuration.IsSet() && Config.Configuration != Configuration.GetValue())
		{
			continue;
		}
		if(ProjectType != EProjectType::Any && Config.ProjectType != EProjectType::Any && Config.ProjectType != ProjectType)
		{
			continue;
		}
		if(State == EInstalledPlatformState::Downloaded && !Config.RequiredFile.IsEmpty() && !FPaths::FileExists(Config.RequiredFile))
		{
			continue;
		}

		// Success!
		return true;
	}

	return false;
}

bool FInstalledPlatformInfo::IsValidPlatformArchitecture(const FString& PlatformName, const FString& Architecture) const
{
	return ContainsValidConfiguration(
		[PlatformName, Architecture](const FInstalledPlatformConfiguration& CurConfig)
		{
			return CurConfig.PlatformName == PlatformName && CurConfig.Architecture.Contains(Architecture);
		}
	);
}

bool FInstalledPlatformInfo::IsPlatformMissingRequiredFile(const FString& PlatformName) const
{
	if (FApp::IsEngineInstalled())
	{
		return ContainsMatchingConfiguration(
			[PlatformName](const FInstalledPlatformConfiguration& CurConfig)
			{
				return CurConfig.PlatformName == PlatformName
					&& !CurConfig.RequiredFile.IsEmpty()
					&& !FPaths::FileExists(CurConfig.RequiredFile);
			}
		);
	}
	return false;
}

bool FInstalledPlatformInfo::OpenInstallerOptions()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();

	if (DesktopPlatform != nullptr && LauncherPlatform != nullptr)
	{
		FString CurrentIdentifier = DesktopPlatform->GetCurrentEngineIdentifier();
		if (DesktopPlatform->IsStockEngineRelease(CurrentIdentifier))
		{
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("NotInstalled_SelectedPlatform", "The Binaries for this Target Platform are not currently installed, would you like to use the Launcher to download them?")) == EAppReturnType::Yes)
			{
				// TODO: Ensure that this URL opens the launcher correctly before this is included in a release
				FString InstallerURL = FString::Printf(TEXT("ue/library/engines/UE_%s/installer"), *DesktopPlatform->GetEngineDescription(CurrentIdentifier));
				FOpenLauncherOptions OpenOptions(InstallerURL);
				if (LauncherPlatform->OpenLauncher(OpenOptions))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FInstalledPlatformInfo::ContainsValidConfiguration(TFunctionRef<bool(const FInstalledPlatformConfiguration)> ConfigFilter) const
{
	if (FApp::IsEngineInstalled())
	{
		for (const FInstalledPlatformConfiguration& PlatformConfiguration : InstalledPlatformConfigurations)
		{
			// Check whether filter accepts this configuration and it has required file
			if (ConfigFilter(PlatformConfiguration)
				&& (PlatformConfiguration.RequiredFile.IsEmpty()
					|| FPaths::FileExists(PlatformConfiguration.RequiredFile)))
			{
				return true;
			}
		}
		return false;
	}
	return true;
}

bool FInstalledPlatformInfo::ContainsMatchingConfiguration(TFunctionRef<bool(const FInstalledPlatformConfiguration)> ConfigFilter) const
{
	if (FApp::IsEngineInstalled())
	{
		for (const FInstalledPlatformConfiguration& PlatformConfiguration : InstalledPlatformConfigurations)
		{
			// Check whether filter accepts this configuration
			if (ConfigFilter(PlatformConfiguration))
			{
				return true;
			}
		}
		return false;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
