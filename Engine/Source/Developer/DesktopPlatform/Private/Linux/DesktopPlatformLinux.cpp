// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/DesktopPlatformLinux.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "DesktopPlatformPrivate.h"
#include "Modules/ModuleManager.h"
#include "Linux/LinuxApplication.h"
#include "Misc/FeedbackContextMarkup.h"
#include "HAL/ThreadHeartBeat.h"
#include "Misc/FileHelper.h"
#include "Internationalization/Regex.h"

//#include "LinuxNativeFeedbackContext.h"
#include "ISlateFileDialogModule.h"
#include "ISlateFontDialogModule.h" 

#define LOCTEXT_NAMESPACE "DesktopPlatform"
#define MAX_FILETYPES_STR 4096
#define MAX_FILENAME_STR 65536

FDesktopPlatformLinux::FDesktopPlatformLinux()
	:	FDesktopPlatformBase()
{
}

FDesktopPlatformLinux::~FDesktopPlatformLinux()
{
}

bool FDesktopPlatformLinux::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->OpenFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, OutFilterIndex);
	}

	return false;
}

bool FDesktopPlatformLinux::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->OpenFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames);
	}

	return false;
}

bool FDesktopPlatformLinux::SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->SaveFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames);
	}

	return false;
}

bool FDesktopPlatformLinux::OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, FString& OutFolderName)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->OpenDirectoryDialog(ParentWindowHandle, DialogTitle, DefaultPath, OutFolderName);
	}

	return false;
}

bool FDesktopPlatformLinux::OpenFontDialog(const void* ParentWindowHandle, FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFontDialog"))
	{
		FModuleManager::Get().LoadModule("SlateFontDialog");
	}

	ISlateFontDialogModule* FontDialog = FModuleManager::GetModulePtr<ISlateFontDialogModule>("SlateFontDialog");
	
	if (FontDialog)
	{
		return FontDialog->OpenFontDialog(OutFontName, OutHeight, OutFlags);
	}
	else
	{
		UE_LOGF(LogLinux, Warning, "Error reading results of font dialog");
	}
	
	return false;
}

bool FDesktopPlatformLinux::FileDialogShared(bool bSave, const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex)
{
	return false;
}

bool FDesktopPlatformLinux::RegisterEngineInstallation(const FString &RootDir, FString &OutIdentifier)
{
	bool bRes = false;
	if (IsValidRootDirectory(RootDir))
	{
		FConfigFile ConfigFile;
		FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / FString(TEXT("UnrealEngine")) / FString(TEXT("Install.ini"));
		ConfigFile.Read(ConfigPath);

		// If this is an installed build, use that Guid instead of generating a new one
		FString InstallationIdPath = FString(RootDir / "Engine" / "Build" / "InstalledBuild.txt");
		FArchive* File = IFileManager::Get().CreateFileReader(*InstallationIdPath, FILEREAD_Silent);
		if(File)
		{
			FFileHelper::LoadFileToString(OutIdentifier, *File);
			OutIdentifier.TrimEndInline();
			FGuid GuidCheck(OutIdentifier);
			if(!GuidCheck.IsValid() && !OutIdentifier.StartsWith(TEXT("UE_")))
			{
				OutIdentifier = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens);
			}

			File->Close();
			delete File;
		}
		else
		{
			OutIdentifier = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens);
		}

		ConfigFile.AddToSection(TEXT("Installations"), *OutIdentifier, RootDir);
		OutIdentifier.RemoveFromStart(TEXT("UE_"));

		ConfigFile.Dirty = true;
		ConfigFile.Write(ConfigPath);
		bRes = true;
	}
	return bRes;
}

void FDesktopPlatformLinux::EnumerateEngineInstallations(TMap<FString, FString> &OutInstallations)
{
	EnumerateLauncherEngineInstallations(OutInstallations);

	FString UProjectPath = FString(FPlatformProcess::ApplicationSettingsDir()) / "Unreal.uproject";
	FArchive* File = IFileManager::Get().CreateFileWriter(*UProjectPath, FILEWRITE_EvenIfReadOnly);
	if (File)
	{
		File->Close();
		delete File;
	}
	else
	{
		FSlowHeartBeatScope SuspendHeartBeat;
		SDL_ShowSimpleMessageBox(SDL_MESSAGEBOX_ERROR, "Unable to write to Settings Directory", TCHAR_TO_UTF8(*UProjectPath), NULL);
	}

	FConfigFile ConfigFile;
	FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / FString(TEXT("UnrealEngine")) / FString(TEXT("Install.ini"));
	ConfigFile.Read(ConfigPath);

	const FConfigSection* Section = ConfigFile.FindOrAddConfigSection(TEXT("Installations"));
	// Remove invalid entries
	// @todo The installations list might contain multiple keys for the same value. Do we have to remove them?
	TArray<FName> KeysToRemove;
	for (auto It : *Section)
	{
		const FString& RootDir = It.Value.GetValue();
		// We remove entries pointing to a folder that doesn't exist or was using the wrong path.
		if (RootDir.Contains(FPaths::EngineDir()) || !IFileManager::Get().DirectoryExists(*RootDir))
		{
			KeysToRemove.Add(It.Key);
			ConfigFile.Dirty = true;
		}
	}
	for (auto Key : KeysToRemove)
	{
		ConfigFile.RemoveKeyFromSection(TEXT("Installations"), Key);
	}

	FConfigSection SectionsToAdd;

	// Iterate through all entries.
	for (auto It : *Section)
	{
		FString NormalizedRootDir = It.Value.GetValue();
		FPaths::NormalizeDirectoryName(NormalizedRootDir);
		FPaths::CollapseRelativeDirectories(NormalizedRootDir);

		FString EngineId;
		const FName* Key = Section->FindKey(NormalizedRootDir);
		if (Key == nullptr)
		{
			Key = SectionsToAdd.FindKey(NormalizedRootDir);
		}

		if (Key)
		{
			FGuid IdGuid;
			FGuid::Parse(Key->ToString(), IdGuid);
			EngineId = IdGuid.ToString(EGuidFormats::DigitsWithHyphens);
		}
		else
		{
			if (!OutInstallations.FindKey(NormalizedRootDir))
			{
				EngineId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens);
				SectionsToAdd.AddUnique(*EngineId, NormalizedRootDir);

				ConfigFile.Dirty = true;
			}
		}

		// local builds only, don't add entries that start with "UE_" which signifies a released build
		if (!EngineId.IsEmpty() && !OutInstallations.Find(EngineId) && !EngineId.StartsWith(TEXT("UE_")))
		{
			OutInstallations.Add(EngineId, NormalizedRootDir);
		}
	}

	for (auto It : SectionsToAdd)
	{
		ConfigFile.AddUniqueToSection(TEXT("Installations"), It.Key, It.Value.GetValue());
	}

	ConfigFile.Write(ConfigPath);

	IFileManager::Get().Delete(*UProjectPath);
}

void FDesktopPlatformLinux::EnumerateLauncherEngineInstallations(TMap<FString, FString> &OutInstallations)
{
	FConfigFile ConfigFile;
	FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / FString(TEXT("UnrealEngine")) / FString(TEXT("Install.ini"));
	ConfigFile.Read(ConfigPath);
	
	const FConfigSection* Section = ConfigFile.FindOrAddConfigSection(TEXT("Installations"));

	FString InstallationIdPath = FString(FPaths::EngineDir() / "Build" / "InstalledBuild.txt");
	FArchive* File = IFileManager::Get().CreateFileReader(*InstallationIdPath, FILEREAD_Silent);
	if (File)
	{
		FString NormalizedRootDir = FPaths::RootDir();
		FPaths::NormalizeDirectoryName(NormalizedRootDir);
		FPaths::CollapseRelativeDirectories(NormalizedRootDir);

		FString Id;
		FFileHelper::LoadFileToString(Id, *File);
		Id.TrimEndInline();

		// if the user unzipped a new installed build into a previously registered directory, we need to fix up the key
		const FName* OldKey = Section->FindKey(NormalizedRootDir);
		if(OldKey && Id != OldKey->ToString())
		{
			ConfigFile.RemoveKeyFromSection(TEXT("Installations"), *OldKey);
			ConfigFile.AddToSection(TEXT("Installations"), *Id, NormalizedRootDir);
			ConfigFile.Write(ConfigPath);
		}
		File->Close();
		delete File;
	}

	// now fill OutInstallations with only released builds
	for (auto It : *Section)
	{
		const FString RootDir = It.Value.GetValue();
		FString GuidOrId = It.Key.ToString();

		// We skip entries pointing to a folder that doesn't exist or was using the wrong path.
		if (RootDir.Contains(FPaths::EngineDir()) || !IFileManager::Get().DirectoryExists(*RootDir))
		{
			continue;
		}

		// released builds only, add entries starting with "UE_"
		if(GuidOrId.RemoveFromStart(TEXT("UE_"), ESearchCase::CaseSensitive))
		{
			if(!OutInstallations.Contains(GuidOrId))
			{
				OutInstallations.Add(*GuidOrId, *RootDir);
			}
		}
	}

}

bool FDesktopPlatformLinux::IsSourceDistribution(const FString &RootDir)
{
	// Check for the existence of a GenerateProjectFiles.sh file. This allows compatibility with the GitHub 4.0 release.
	FString GenerateProjectFilesPath = RootDir / TEXT("GenerateProjectFiles.sh");
	if (IFileManager::Get().FileSize(*GenerateProjectFilesPath) >= 0)
	{
		return true;
	}

	// Otherwise use the default test
	return FDesktopPlatformBase::IsSourceDistribution(RootDir);
}

static bool RunXDGUtil(FString XDGUtilCommand, FString* StdOut = nullptr)
{
	// Run through bash incase xdg-utils is overriden via path.
	FString CommandLine = TEXT("/bin/bash");

	int32 ReturnCode;
	if (FPlatformProcess::ExecProcess(*CommandLine, *XDGUtilCommand, &ReturnCode, StdOut, nullptr) && ReturnCode == 0)
	{
		return true;
	}

	return false;
}

static bool CompareAndCheckDesktopFile(const TCHAR* DesktopFileName, const TCHAR* MimeType)
{
	FString Association(DesktopFileName);
	if (MimeType != nullptr)
	{
		Association = FString();
		RunXDGUtil(*FString::Printf(TEXT("xdg-mime query default %s"), MimeType), &Association);
		if (!Association.Contains(TEXT(".desktop")))
		{
			return false;
		}
		Association = Association.Replace(TEXT(".desktop"), TEXT(""));
		Association = Association.Replace(TEXT("\n"), TEXT(""));
	}

	// There currently appears to be no way to locate the desktop file with xdg-utils so access the file via the expected location.
	FString DataDir = FPlatformMisc::GetEnvironmentVariable(TEXT("XDG_DATA_HOME"));
	if (DataDir.Len() == 0)
	{	
		DataDir = FPlatformMisc::GetEnvironmentVariable(TEXT("HOME")) + TEXT("/.local/share");
	}

	// Get the contents of the desktop file.
	FString InstalledDesktopFileContents;
	FFileHelper::LoadFileToString(InstalledDesktopFileContents, *FString::Printf(TEXT("%s/applications/%s.desktop"), *DataDir, *Association));

	// Make sure the installed and default desktop file was created by unreal engine.
	if (!InstalledDesktopFileContents.Contains(TEXT("Comment=Created by Unreal Engine")))
	{
		return false;
	}

	// Get the version of the installed desktop file.
	float InstalledVersion = 0.0;
	FRegexPattern Pattern(TEXT("Version=(.*)\\n"));
	FRegexMatcher Matcher(Pattern, InstalledDesktopFileContents);
	const TCHAR* Contents = *InstalledDesktopFileContents;
	if (Matcher.FindNext())
	{
		InstalledVersion = FCString::Atof(*Matcher.GetCaptureGroup(1));
	}
	else
	{
		return false;
	}

	// Get the version of the template desktop file for this engine source.
	FString TemplateDesktopFileContents;
	float TemplateVersion = 0.0;
	FFileHelper::LoadFileToString(TemplateDesktopFileContents, *FString::Printf(TEXT("%sPrograms/UnrealVersionSelector/Private/Linux/Resources/%s.desktop"), *FPaths::EngineSourceDir(), DesktopFileName));
	Matcher = FRegexMatcher(Pattern, TemplateDesktopFileContents);
	if (Matcher.FindNext())
	{
		TemplateVersion = FCString::Atof(*Matcher.GetCaptureGroup(1));
	}

	// If our template version is greater than the installed version then it needs to be updated to point to this engine's version.
	if (TemplateVersion > InstalledVersion)
	{
		return false;
	}

	// If the template version was lower or the same check if the installed version points to a valid binary.	
	FString DesktopFileExecPath;
	Pattern = FRegexPattern(TEXT("Exec=(.*) %f\\n"));
	Matcher = FRegexMatcher(Pattern, TemplateDesktopFileContents);
	if (Matcher.FindNext())
	{
		DesktopFileExecPath = Matcher.GetCaptureGroup(1);
	}

	if (DesktopFileExecPath.Compare("bash") != 0 && !FPaths::FileExists(DesktopFileExecPath))
	{
		return false;
	}

	return true;
}

bool FDesktopPlatformLinux::VerifyFileAssociations()
{
	if (!CompareAndCheckDesktopFile(TEXT("com.epicgames.UnrealVersionSelector"), TEXT("application/uproject")))
	{
		return false;
	}

	if (!CompareAndCheckDesktopFile(TEXT("com.epicgames.UnrealEngine"), nullptr))
	{
		return false;
	}

	return true;
}

bool FDesktopPlatformLinux::UpdateFileAssociations()
{
	// It would be more robust to follow the XDG spec and alter the mime and desktop databases directly.
	// However calling though to xdg-utils provides a simpler implementation and allows a user or distro to override the scripts.
	if (VerifyFileAssociations())
	{
		// If UVS was already installed and the same version or greater then it should not be updated.
		return true;
	}

	// Install the png icons, one for uprojects and one for the main Unreal Engine launcher.
	if (!RunXDGUtil(FString::Printf(TEXT("xdg-icon-resource install --novendor --mode user --context mimetypes --size 256 %sPrograms/UnrealVersionSelector/Private/Linux/Resources/Icon.png uproject"), *FPaths::EngineSourceDir())))
	{
		return false;
	}

	if (!RunXDGUtil(FString::Printf(TEXT("xdg-icon-resource install --novendor --mode user --context apps --size 256 %sRuntime/Launch/Resources/Linux/UnrealEngine.png ubinary"), *FPaths::EngineSourceDir())))
	{
		return false;
	}

	FString IconSource = FPaths::Combine(FPaths::EngineSourceDir(), TEXT("Programs/UnrealVersionSelector/Private/Linux/Resources/Icon.svg")); 
	FString ProjectIconDestination = FPaths::Combine(FPlatformMisc::GetEnvironmentVariable(TEXT("HOME")), TEXT(".local/share/icons/hicolor/scalable/mimetypes")); 
	FString BinaryIconDestination  = FPaths::Combine(FPlatformMisc::GetEnvironmentVariable(TEXT("HOME")), TEXT(".local/share/icons/hicolor/scalable/apps")); 

	IFileManager& FileManager = IFileManager::Get();

	// Ensure that the proper directories exist for svg icons as well
	if (!FileManager.DirectoryExists(*ProjectIconDestination))
	{
		if (!FileManager.MakeDirectory(*ProjectIconDestination))
		{
			return false;
		}
	}
	if (!FileManager.DirectoryExists(*BinaryIconDestination))
	{
		if (!FileManager.MakeDirectory(*BinaryIconDestination))
		{
			return false;
		}
	}

	// Install the svg icons; this is done manually because xdg-icon-resource doesn't support installation of svg files
	if (FileManager.Copy(*FPaths::Combine(ProjectIconDestination, TEXT("uproject.svg")), *IconSource) != COPY_OK)
	{
		return false;
	}
	if (FileManager.Copy(*FPaths::Combine(BinaryIconDestination, TEXT("ubinary.svg")), *IconSource) != COPY_OK)
	{
		return false;
	}

	FString AbsoluteEngineDir = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::EngineDir());

	// Add the desktop file for the Unreal Version Selector mime-type from the template.
	FString DesktopTemplate;
	FFileHelper::LoadFileToString(DesktopTemplate, *FString::Printf(TEXT("%sPrograms/UnrealVersionSelector/Private/Linux/Resources/com.epicgames.UnrealVersionSelector.desktop"), *FPaths::EngineSourceDir()));        
	DesktopTemplate = DesktopTemplate.Replace(TEXT("*ENGINEDIR*"), *AbsoluteEngineDir);
	FFileHelper::SaveStringToFile(DesktopTemplate, TEXT("/tmp/com.epicgames.UnrealVersionSelector.desktop"));
	if (!RunXDGUtil(TEXT("xdg-desktop-menu install --novendor --mode user /tmp/com.epicgames.UnrealVersionSelector.desktop")))
	{
		return false;
	}

	// Add the desktop file for the Unreal Engine Generate Project List icon from the template.
	DesktopTemplate = FString();
	FFileHelper::LoadFileToString(DesktopTemplate, *FString::Printf(TEXT("%sPrograms/UnrealVersionSelector/Private/Linux/Resources/com.epicgames.UnrealEngine.desktop"), *FPaths::EngineSourceDir()));      
	DesktopTemplate = DesktopTemplate.Replace(TEXT("*ENGINEDIR*"), *AbsoluteEngineDir);
	FFileHelper::SaveStringToFile(DesktopTemplate, TEXT("/tmp/com.epicgames.UnrealEngine.desktop"));
	if (!RunXDGUtil(TEXT("xdg-desktop-menu install --novendor --mode user /tmp/com.epicgames.UnrealEngine.desktop")))
	{
		return false;
	}

	// Add the desktop file for the Unreal Engine Editor icon from the template.
	DesktopTemplate = FString();
	FFileHelper::LoadFileToString(DesktopTemplate, *FString::Printf(TEXT("%sPrograms/UnrealVersionSelector/Private/Linux/Resources/com.epicgames.UnrealEngineEditor.desktop"), *FPaths::EngineSourceDir()));      
	DesktopTemplate = DesktopTemplate.Replace(TEXT("*ENGINEDIR*"), *AbsoluteEngineDir);
	FFileHelper::SaveStringToFile(DesktopTemplate, TEXT("/tmp/com.epicgames.UnrealEngineEditor.desktop"));
	if (!RunXDGUtil(TEXT("xdg-desktop-menu install --novendor --mode user /tmp/com.epicgames.UnrealEngineEditor.desktop")))
	{
		return false;
	}

	// Create the mime types and set the default applications.
	if (!RunXDGUtil(FString::Printf(TEXT("xdg-mime install --novendor --mode user %sPrograms/UnrealVersionSelector/Private/Linux/Resources/uproject.xml"), *FPaths::EngineSourceDir())))
	{
		return false;
	}
	if (!RunXDGUtil(TEXT("xdg-mime default com.epicgames.UnrealEngineEditor.desktop application/uproject")))
	{
		return false;
	}

	return true;
}

bool FDesktopPlatformLinux::OpenProject(const FString &ProjectFileName)
{
	// Get the project filename in a native format
	FString PlatformProjectFileName = ProjectFileName;
	FPaths::MakePlatformFilename(PlatformProjectFileName);

	STUBBED("FDesktopPlatformLinux::OpenProject");
	return false;
}

bool FDesktopPlatformLinux::RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutExitCode)
{
	OutExitCode = 1;

	// Get the path to UBT
	FString UnrealBuildToolPath = GetUnrealBuildToolExecutableFilename(RootDir);
	if(IFileManager::Get().FileSize(*UnrealBuildToolPath) < 0)
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("Couldn't find UnrealBuildTool at '%s'"), *UnrealBuildToolPath);
		return false;
	}

	// Write the output
	Warn->Logf(TEXT("Running %s %s"), *UnrealBuildToolPath, *Arguments);

	return FFeedbackContextMarkup::PipeProcessOutput(Description, UnrealBuildToolPath, Arguments, Warn, &OutExitCode) && OutExitCode == 0;
}

FFeedbackContext* FDesktopPlatformLinux::GetNativeFeedbackContext()
{
	//unimplemented();
	STUBBED("FDesktopPlatformLinux::GetNativeFeedbackContext");
	return GWarn;
}

FString FDesktopPlatformLinux::GetUserTempPath()
{
	return FString(FPlatformProcess::UserTempDir());
}

FString FDesktopPlatformLinux::GetOidcTokenExecutableFilename(const FString& RootDir) const
{	
	return FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Binaries/DotNET/OidcToken/linux-x64/OidcToken"));
}

#undef LOCTEXT_NAMESPACE
