// Copyright Epic Games, Inc. All Rights Reserved.

#include "LockFile.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Memory/SharedBuffer.h"
#include "Misc/FileHelper.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"


#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include <shellapi.h>
#	include <synchapi.h>
#	include <Windows.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

#if PLATFORM_UNIX || PLATFORM_MAC
#	include <sys/file.h>
#	include <sys/sem.h>
#	include <sys/stat.h>
#	include <unistd.h>
#endif

bool
FLockFile::TryReadAndClear(const TCHAR* FileName, FString& OutContents)
{
	OutContents.Reset();
#if PLATFORM_WINDOWS
	uint32 Access = GENERIC_READ | DELETE;
	uint32 WinFlags = 0; // Exclusive access
	uint32 Disposition = OPEN_EXISTING;

	TStringBuilder<MAX_PATH> FullFileNameBuilder;
	FPathViews::ToAbsolutePath(FStringView(FileName), FullFileNameBuilder);
	for (TCHAR& Char : MakeArrayView(FullFileNameBuilder))
	{
		if (Char == TEXT('/'))
		{
			Char = TEXT('\\');
		}
	}
	if (FullFileNameBuilder.Len() >= MAX_PATH)
	{
		FullFileNameBuilder.Prepend(TEXTVIEW("\\\\?\\"));
	}
	HANDLE Handle = CreateFileW(*FullFileNameBuilder, Access, WinFlags, NULL, Disposition, FILE_ATTRIBUTE_NORMAL, NULL);
	if (Handle != INVALID_HANDLE_VALUE)
	{
		ON_SCOPE_EXIT { CloseHandle(Handle); };
		LARGE_INTEGER LI;
		if (GetFileSizeEx(Handle, &LI))
		{
			checkf(LI.QuadPart == LI.u.LowPart, TEXT("File exceeds supported 2GB limit."));
			int32 FileSize32 = LI.u.LowPart;
			FUniqueBuffer FileBytes = FUniqueBuffer::Alloc(FileSize32);
			DWORD ReadBytes = 0;
			if (ReadFile(Handle, FileBytes.GetData(), FileSize32, &ReadBytes, NULL) && (ReadBytes == FileSize32))
			{
				FFileHelper::BufferToString(OutContents, reinterpret_cast<const uint8*>(FileBytes.GetData()), FileSize32);
				// Set the 'DeleteFile' disposition on the handle to delete it on close
				FILE_DISPOSITION_INFO DispositionInfo { 1 };
				SetFileInformationByHandle(Handle, FileDispositionInfo, &DispositionInfo, sizeof(DispositionInfo));
				return true;
			}
		}
	}
	return false;
#elif PLATFORM_UNIX || PLATFORM_MAC
	TAnsiStringBuilder<256> FilePath;
	FilePath << FileName;
	int32 Fd = open(FilePath.ToString(), O_RDONLY);
	if (Fd < 0)
	{
		return false;
	}

	int32 LockRet = flock(Fd, LOCK_EX | LOCK_NB);
	if (LockRet < 0)
	{
		close(Fd);
		return false;
	}

	ON_SCOPE_EXIT
	{
		flock(Fd, LOCK_UN);
		close(Fd);
	};

	struct stat Stat;
	fstat(Fd, &Stat);
	uint64 FileSize = uint64(Stat.st_size);

	bool bSuccess = false;
	FUniqueBuffer FileBytes = FUniqueBuffer::Alloc(FileSize);
	if (read(Fd, FileBytes.GetData(), FileSize) == FileSize)
	{
		FFileHelper::BufferToString(OutContents, reinterpret_cast<const uint8*>(FileBytes.GetData()), FileSize);
	}

	unlink(FilePath.ToString());

	return true;
#else
	unimplemented();
	return false;
#endif
}