// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/DesktopPlatformMac.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "Mac/MacApplication.h"
#include "Misc/FeedbackContextMarkup.h"
#include "MacNativeFeedbackContext.h"
#include "Mac/CocoaThread.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "HAL/FileManager.h"
#import <UniformTypeIdentifiers/UniformTypeIdentifiers.h>

#define LOCTEXT_NAMESPACE "DesktopPlatform"

class FMacScopedSystemModalMode
{
public:
	FMacScopedSystemModalMode()
	{
		MacApplication->SystemModalMode(true);
	}
	
	~FMacScopedSystemModalMode()
	{
		MacApplication->SystemModalMode(false);
	}
private:
	FScopedSystemModalMode SystemModalMode;
};

class FCocoaScopeContext
{
public:
	FCocoaScopeContext(void)
	{
		SCOPED_AUTORELEASE_POOL;
		PreviousContext = [NSOpenGLContext currentContext];
	}
	
	~FCocoaScopeContext( void )
	{
		SCOPED_AUTORELEASE_POOL;
		NSOpenGLContext* NewContext = [NSOpenGLContext currentContext];
		if (PreviousContext != NewContext)
		{
			if (PreviousContext)
			{
				[PreviousContext makeCurrentContext];
			}
			else
			{
				[NSOpenGLContext clearCurrentContext];
			}
		}
	}
	
private:
	NSOpenGLContext*	PreviousContext;
};

/**
 * Custom accessory view class to allow choose kind of file extension
 */
@interface FFileDialogAccessoryView : NSView
{
@private
	NSPopUpButton*	PopUpButton;
	NSTextField*	TextField;
	NSSavePanel*	DialogPanel;
	NSMutableArray*	AllowedFileTypes;
	int32			SelectedExtension;
}

- (id)initWithFrame:(NSRect)frameRect dialogPanel:(NSSavePanel*) panel;
- (void)PopUpButtonAction: (id) sender;
- (void)AddAllowedFileTypes: (NSArray*) array;
- (void)SetExtensionsAtIndex: (int32) index;
- (int32)SelectedExtension;

@end

@implementation FFileDialogAccessoryView

- (id)initWithFrame:(NSRect)frameRect dialogPanel:(NSSavePanel*) panel
{
	self = [super initWithFrame: frameRect];
	DialogPanel = panel;
	
	FString FieldText = LOCTEXT("FileExtension", "File extension:").ToString();
	CFStringRef FieldTextCFString = FPlatformString::TCHARToCFString(*FieldText);
	TextField = [[NSTextField alloc] initWithFrame: NSMakeRect(0.0, 48.0, 90.0, 25.0) ];
	[TextField setStringValue:(NSString*)FieldTextCFString];
	[TextField setEditable:NO];
	[TextField setBordered:NO];
	[TextField setBackgroundColor:[NSColor controlColor]];
	

	PopUpButton = [[NSPopUpButton alloc] initWithFrame: NSMakeRect(88.0, 50.0, 160.0, 25.0) ];
	[PopUpButton setTarget: self];
	[PopUpButton setAction:@selector(PopUpButtonAction:)];

	[self addSubview: TextField];
	[self addSubview: PopUpButton];

	return self;
}

- (void)AddAllowedFileTypes: (NSMutableArray*) array
{
	check( array );

	AllowedFileTypes = array;
	int32 ArrayCount = [AllowedFileTypes count];
	if( ArrayCount )
	{
		check( ArrayCount % 2 == 0 );

		[PopUpButton removeAllItems];

		for( int32 Index = 0; Index < ArrayCount; Index += 2 )
		{
			[PopUpButton addItemWithTitle: [AllowedFileTypes objectAtIndex: Index]];
		}

		// Set allowed extensions
		[self SetExtensionsAtIndex: 0];
	}
	else
	{
		// Allow all file types (empty array means all types)
		DialogPanel.allowedContentTypes = @[];
	}
}

- (void)PopUpButtonAction: (id) sender
{
	NSInteger Index = [PopUpButton indexOfSelectedItem];
	[self SetExtensionsAtIndex: Index];
}

- (void)SetExtensionsAtIndex: (int32) index
{
	check( [AllowedFileTypes count] >= index * 2 );
	SelectedExtension = index;

	NSString* ExtsToParse = [AllowedFileTypes objectAtIndex:index * 2 + 1];
	if( [ExtsToParse compare:@"*.*"] == NSOrderedSame )
	{
		// Allow all file types (empty array means all types)
		DialogPanel.allowedContentTypes = @[];
	}
	else
	{
		NSArray* ExtensionsWildcards = [ExtsToParse componentsSeparatedByString:@";"];
		NSMutableArray* Extensions = [NSMutableArray arrayWithCapacity: [ExtensionsWildcards count]];

		for( int32 Index = 0; Index < [ExtensionsWildcards count]; ++Index )
		{
			// capture potential leading whitespace and *.
			NSMutableCharacterSet* TrimSet = [NSMutableCharacterSet characterSetWithCharactersInString:@"*."];                                                    
			[TrimSet formUnionWithCharacterSet:[NSCharacterSet whitespaceCharacterSet]];                                                                          
			NSString* Temp = [[ExtensionsWildcards objectAtIndex:Index] stringByTrimmingCharactersInSet:TrimSet];
			
			UTType* Type = [UTType typeWithFilenameExtension:Temp];
			
			if(Type == nil)
			{
				UE_LOGF(LogMac, Warning, "Couldn't find UTType for requested file picker extension extension %hs", [Temp UTF8String]);
			}
			else
			{
				[Extensions addObject: Type];
			}
		}

		DialogPanel.allowedContentTypes = Extensions;
	}
}

- (int32)SelectedExtension {
    return SelectedExtension;
}

@end

/**
 * Custom accessory view class to allow choose kind of file extension
 */
@interface FFontDialogAccessoryView : NSView
{
@private

	NSButton*	OKButton;
	NSButton*	CancelButton;
	bool		Result;
}

- (id)initWithFrame: (NSRect)frameRect;
- (bool)result;
- (IBAction)onCancel: (id)sender;
- (IBAction)onOK: (id)sender;

@end

@implementation FFontDialogAccessoryView : NSView

- (id)initWithFrame: (NSRect)frameRect
{
	[super initWithFrame: frameRect];

	CancelButton = [[NSButton alloc] initWithFrame: NSMakeRect(10, 10, 80, 24)];
	[CancelButton setTitle: @"Cancel"];
	[CancelButton setBezelStyle: NSBezelStyleRounded];
	[CancelButton setButtonType: NSButtonTypeMomentaryPushIn];
	[CancelButton setAction: @selector(onCancel:)];
	[CancelButton setTarget: self];
	[self addSubview: CancelButton];

	OKButton = [[NSButton alloc] initWithFrame: NSMakeRect(100, 10, 80, 24)];
	[OKButton setTitle: @"OK"];
	[OKButton setBezelStyle: NSBezelStyleRounded];
	[OKButton setButtonType: NSButtonTypeMomentaryPushIn];
	[OKButton setAction: @selector(onOK:)];
	[OKButton setTarget: self];
	[self addSubview: OKButton];

	Result = false;

	return self;
}

- (bool)result
{
	return Result;
}

- (IBAction)onCancel: (id)sender
{
	Result = false;
	[NSApp stopModal];
}

- (IBAction)onOK: (id)sender
{
	Result = true;
	[NSApp stopModal];
}

@end

bool FDesktopPlatformMac::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	int32 DummyIdx = 0;
	return FileDialogShared(false, ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyIdx);
}

bool FDesktopPlatformMac::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex)
{
	return FileDialogShared(false, ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, OutFilterIndex);
}

bool FDesktopPlatformMac::SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	int32 DummyIdx = 0;
	return FileDialogShared(true, ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyIdx);
}

bool FDesktopPlatformMac::OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, FString& OutFolderName)
{
	MacApplication->SetCapture( NULL );

	bool bSuccess = false;
	{
		FMacScopedSystemModalMode SystemModalScope;
		bSuccess = MainThreadReturn(^{
			SCOPED_AUTORELEASE_POOL;
			FCocoaScopeContext ContextGuard;

			NSOpenPanel* Panel = [NSOpenPanel openPanel];
			[Panel setCanChooseFiles: false];
			[Panel setCanChooseDirectories: true];
			[Panel setAllowsMultipleSelection: false];
			[Panel setCanCreateDirectories: true];

			CFStringRef Title = FPlatformString::TCHARToCFString(*DialogTitle);
			[Panel setTitle: (NSString*)Title];
			CFRelease(Title);

			CFStringRef DefaultPathCFString = FPlatformString::TCHARToCFString(*DefaultPath);
			NSURL* DefaultPathURL = [NSURL fileURLWithPath: (NSString*)DefaultPathCFString];
			[Panel setDirectoryURL: DefaultPathURL];
			CFRelease(DefaultPathCFString);

			bool bResult = false;

			NSInteger Result = [Panel runModal];

			if (Result == NSModalResponseOK)
			{
				NSURL *FolderURL = [[Panel URLs] objectAtIndex: 0];
				TCHAR FolderPath[MAC_MAX_PATH];
				FPlatformString::CFStringToTCHAR((CFStringRef)[FolderURL path], FolderPath);
				OutFolderName = FolderPath;
				FPaths::NormalizeFilename(OutFolderName);

				bResult = true;
			}

			[Panel close];

			return bResult;
		});
	}
	MacApplication->ResetModifierKeys();
	
	return bSuccess;
}

bool FDesktopPlatformMac::OpenFontDialog(const void* ParentWindowHandle, FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags)
{
	MacApplication->SetCapture( NULL );
	
	bool bSuccess = false;
	{
		FMacScopedSystemModalMode SystemModalScope;
		bSuccess = MainThreadReturn(^{
			SCOPED_AUTORELEASE_POOL;
			FCocoaScopeContext ContextGuard;

			NSFontPanel* Panel = [NSFontPanel sharedFontPanel];
			[Panel setFloatingPanel: false];
			[[Panel standardWindowButton: NSWindowCloseButton] setEnabled: false];
			FFontDialogAccessoryView* AccessoryView = [[FFontDialogAccessoryView alloc] initWithFrame: NSMakeRect(0.0, 0.0, 190.0, 80.0)];
			[Panel setAccessoryView: AccessoryView];

			[NSApp runModalForWindow: Panel];
			
			[Panel close];

			bool bResult = [AccessoryView result];

			[Panel setAccessoryView: NULL];
			[AccessoryView release];
			[[Panel standardWindowButton: NSWindowCloseButton] setEnabled: true];

			if( bResult )
			{
				NSFont* Font = [Panel panelConvertFont: [NSFont userFontOfSize: 0]];

				TCHAR FontName[MAC_MAX_PATH];
				FPlatformString::CFStringToTCHAR((CFStringRef)[Font fontName], FontName);

				OutFontName = FontName;
				OutHeight = (float)[Font pointSize];

				auto FontFlags = EFontImportFlags::None;

				if( [Font underlineThickness] >= 1.0 )
				{
					FontFlags |= EFontImportFlags::EnableUnderline;
				}

				OutFlags = FontFlags;
			}

			return bResult;
		});
	}
	
	MacApplication->ResetModifierKeys();
	
	return bSuccess;
}

bool FDesktopPlatformMac::FileDialogShared(bool bSave, const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex)
{
	MacApplication->SetCapture( NULL );

	bool bSuccess = false;
	{
		FMacScopedSystemModalMode SystemModalScope;
		bSuccess = MainThreadReturn(^{
			SCOPED_AUTORELEASE_POOL;
			FCocoaScopeContext ContextGuard;

			NSSavePanel* Panel = bSave ? [NSSavePanel savePanel] : [NSOpenPanel openPanel];

			if (!bSave)
			{
				NSOpenPanel* OpenPanel = (NSOpenPanel*)Panel;
				[OpenPanel setCanChooseFiles: true];
				[OpenPanel setCanChooseDirectories: false];
				[OpenPanel setAllowsMultipleSelection: (Flags & EFileDialogFlags::Multiple) != 0];
			}

			[Panel setCanCreateDirectories: bSave];

			CFStringRef Title = FPlatformString::TCHARToCFString(*DialogTitle);
			[Panel setTitle: (NSString*)Title];
			CFRelease(Title);

			CFStringRef DefaultPathCFString = FPlatformString::TCHARToCFString(*DefaultPath);
			NSURL* DefaultPathURL = [NSURL fileURLWithPath: (NSString*)DefaultPathCFString];
			
			if(bSave)
			{
				// If pointing to a file then remove file from the path - Save Panel is expecting a directory URL
				// Works round crash in Catalina and error messages in Big Sur
				NSNumber* isDirectoryKey;
				if([DefaultPathURL getResourceValue:&isDirectoryKey forKey:NSURLIsDirectoryKey error:nil] == NO || [isDirectoryKey boolValue] == NO)
				{
					DefaultPathURL = [DefaultPathURL URLByDeletingLastPathComponent];
				}
			}
			
			[Panel setDirectoryURL: DefaultPathURL];
			CFRelease(DefaultPathCFString);

			CFStringRef FileNameCFString = FPlatformString::TCHARToCFString(*DefaultFile);
			[Panel setNameFieldStringValue: (NSString*)FileNameCFString];
			CFRelease(FileNameCFString);

			FFileDialogAccessoryView* AccessoryView = [[FFileDialogAccessoryView alloc] initWithFrame: NSMakeRect( 0.0, 0.0, 250.0, 85.0 ) dialogPanel: Panel];
			[Panel setAccessoryView: AccessoryView];

			TArray<FString> FileTypesArray;
			int32 NumFileTypes = FileTypes.ParseIntoArray(FileTypesArray, TEXT("|"), true);

			NSMutableArray* AllowedFileTypes = [NSMutableArray arrayWithCapacity: NumFileTypes];

			if( NumFileTypes > 0 )
			{
				for( int32 Index = 0; Index < NumFileTypes; ++Index )
				{
					CFStringRef Type = FPlatformString::TCHARToCFString(*FileTypesArray[Index]);
					[AllowedFileTypes addObject: (NSString*)Type];
					CFRelease(Type);
				}
			}
			
			if(AllowedFileTypes != nil)
			{
				if ([AllowedFileTypes count] == 1)
				{
					// Duplicate the type for the description
					[AllowedFileTypes addObject:AllowedFileTypes[0]];
				}
				
				[AccessoryView AddAllowedFileTypes:AllowedFileTypes];
			}

			bool bOkPressed = false;
			NSWindow* FocusWindow = [[NSApplication sharedApplication] keyWindow];

			NSInteger Result = [Panel runModal];
			[AccessoryView release];

			if (Result == NSModalResponseOK)
			{
				if (bSave)
				{
					TCHAR FilePath[MAC_MAX_PATH];
					FPlatformString::CFStringToTCHAR((CFStringRef)[[Panel URL] path], FilePath);
					new(OutFilenames) FString(FilePath);
				}
				else
				{
					NSOpenPanel* OpenPanel = (NSOpenPanel*)Panel;
					for (NSURL *FileURL in [OpenPanel URLs])
					{
						TCHAR FilePath[MAC_MAX_PATH];
						FPlatformString::CFStringToTCHAR((CFStringRef)[FileURL path], FilePath);
						new(OutFilenames) FString(FilePath);
					}
					OutFilterIndex = [AccessoryView SelectedExtension];
				}

				// Make sure all filenames gathered have their paths normalized
				for (auto OutFilenameIt = OutFilenames.CreateIterator(); OutFilenameIt; ++OutFilenameIt)
				{
					FString& OutFilename = *OutFilenameIt;
					OutFilename = IFileManager::Get().ConvertToRelativePath(*OutFilename);
					FPaths::NormalizeFilename(OutFilename);
				}

				bOkPressed = true;
			}

			[Panel close];
			
			if(FocusWindow)
			{
				[FocusWindow makeKeyWindow];
			}

			return bOkPressed;
		});
	}
	
	MacApplication->ResetModifierKeys();

	return bSuccess;
}

bool FDesktopPlatformMac::RegisterEngineInstallation(const FString &RootDir, FString &OutIdentifier)
{
	bool bRes = false;
	if (IsValidRootDirectory(RootDir))
	{
		FConfigFile ConfigFile;
		FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / FString(TEXT("UnrealEngine")) / FString(TEXT("Install.ini"));
		ConfigFile.Read(ConfigPath);

		OutIdentifier = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensInBraces);
		ConfigFile.AddToSection(TEXT("Installations"), *OutIdentifier, RootDir);

		ConfigFile.Write(ConfigPath);
	}
	return bRes;
}

void FDesktopPlatformMac::EnumerateEngineInstallations(TMap<FString, FString> &OutInstallations)
{
	SCOPED_AUTORELEASE_POOL;
	EnumerateLauncherEngineInstallations(OutInstallations);

	// Create temp .uproject file to use with LSCopyApplicationURLsForURL
	FString UProjectPath = FString(FPlatformProcess::ApplicationSettingsDir()) / "Unreal.uproject";
	FArchive* File = IFileManager::Get().CreateFileWriter(*UProjectPath, FILEWRITE_EvenIfReadOnly);
	if (File)
	{
		File->Close();
		delete File;
	}
	else
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *UProjectPath, TEXT("Error"));
	}

	FConfigFile ConfigFile;
	FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / FString(TEXT("UnrealEngine")) / FString(TEXT("Install.ini"));
	ConfigFile.Read(ConfigPath);

	const FConfigSection *Section = ConfigFile.FindOrAddConfigSection(TEXT("Installations"));
	// Remove invalid entries
	TArray<FName> KeysToRemove;
	for (const TPair<FName, FConfigValue>& It : *Section)
	{
		const FString& EngineDir = It.Value.GetValue();
		if (EngineDir.Contains("Unreal Engine.app/Contents/") || EngineDir.Contains("Epic Games Launcher.app/Contents/") || EngineDir.Contains("/Users/Shared/UnrealEngine/Launcher") || !IFileManager::Get().DirectoryExists(*EngineDir))
		{
			KeysToRemove.Add(It.Key);
		}
	}
	for (FName Key : KeysToRemove)
	{
		ConfigFile.RemoveKeyFromSection(TEXT("Installations"), Key);
	}

	CFArrayRef AllApps = LSCopyApplicationURLsForURL((__bridge CFURLRef)[NSURL fileURLWithPath:UProjectPath.GetNSString()], kLSRolesAll);
	if (AllApps)
	{
		const CFIndex AppsCount = CFArrayGetCount(AllApps);
		for (CFIndex Index = 0; Index < AppsCount; ++Index)
		{
			NSURL* AppURL = (NSURL*)CFArrayGetValueAtIndex(AllApps, Index);
			NSBundle* AppBundle = [NSBundle bundleWithURL:AppURL];
			FString EngineDir = FString([[AppBundle bundlePath] stringByDeletingLastPathComponent]);
			if (([[AppBundle bundleIdentifier] isEqualToString:@"com.epicgames.UnrealEditor"] || [[AppBundle bundleIdentifier] isEqualToString:@"com.epicgames.UnrealEditorServices"])
				&& EngineDir.RemoveFromEnd(TEXT("/Engine/Binaries/Mac")) && !EngineDir.Contains("Unreal Engine.app/Contents/") && !EngineDir.Contains("Epic Games Launcher.app/Contents/") && !EngineDir.Contains("/Users/Shared/UnrealEngine/Launcher"))
			{
				FString EngineId;
				const FName* Key = Section->FindKey(EngineDir);
				if (Key)
				{
					FGuid IdGuid;
					FGuid::Parse(Key->ToString(), IdGuid);
					EngineId = IdGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces);;
				}
				else
				{
					if (!OutInstallations.FindKey(EngineDir))
					{
						EngineId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensInBraces);
						ConfigFile.AddToSection(TEXT("Installations"), *EngineId, EngineDir);
					}
				}
				if (!EngineId.IsEmpty() && !OutInstallations.Find(EngineId))
				{
					OutInstallations.Add(EngineId, EngineDir);
				}
			}
		}

		ConfigFile.Write(ConfigPath);
		CFRelease(AllApps);
	}

	IFileManager::Get().Delete(*UProjectPath);
}

bool FDesktopPlatformMac::VerifyFileAssociations()
{
	CFURLRef GlobalDefaultAppURL = NULL;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OSStatus Status = LSGetApplicationForInfo(kLSUnknownType, kLSUnknownCreator, CFSTR("uproject"), kLSRolesAll, NULL, &GlobalDefaultAppURL);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (Status == noErr)
	{
		NSBundle* GlobalDefaultAppBundle = [NSBundle bundleWithURL:(__bridge NSURL*)GlobalDefaultAppURL];
		CFRelease(GlobalDefaultAppURL);

		if ([[GlobalDefaultAppBundle bundleIdentifier] isEqualToString:@"com.epicgames.UnrealEditorServices"])
		{
			return true;
		}
	}
	return false;
}

bool FDesktopPlatformMac::UpdateFileAssociations()
{
	OSStatus Status = LSSetDefaultRoleHandlerForContentType(CFSTR("com.epicgames.uproject"), kLSRolesAll, CFSTR("com.epicgames.UnrealEditorServices"));
	return Status == noErr;
}

bool FDesktopPlatformMac::RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutExitCode)
{
	OutExitCode = 1;

	// Get the path to UBT
	FString UnrealBuildToolPath = GetUnrealBuildToolExecutableFilename(RootDir);
	if(IFileManager::Get().FileSize(*UnrealBuildToolPath) < 0)
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("Couldn't find UnrealBuildTool at '%s'"), *UnrealBuildToolPath);
		return false;
	}

	// Spawn it
	return FFeedbackContextMarkup::PipeProcessOutput(Description, UnrealBuildToolPath, Arguments, Warn, &OutExitCode) && OutExitCode == 0;
}

FFeedbackContext* FDesktopPlatformMac::GetNativeFeedbackContext()
{
	static FMacNativeFeedbackContext Warn;
	return &Warn;
}

FString FDesktopPlatformMac::GetUserTempPath()
{
	return FString(FPlatformProcess::UserTempDir());
}

FString FDesktopPlatformMac::GetOidcTokenExecutableFilename(const FString& RootDir) const
{
	return FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Binaries/DotNET/OidcToken/osx-x64/OidcToken"));
}
#undef LOCTEXT_NAMESPACE
