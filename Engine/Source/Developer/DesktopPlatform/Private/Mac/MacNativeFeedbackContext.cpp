// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacNativeFeedbackContext.h"

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"
#include "HAL/PlatformTime.h"
#include "Logging/StructuredLog.h"
#include "Mac/MacApplication.h"
#include "Mac/CocoaThread.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/PlatformApplicationMisc.h"

@implementation FMacNativeFeedbackContextWindowController

-(void)toggleLog
{
	if([LogView isHidden])
	{
		NSRect Frame = [Window frame];
		
		int32 ConsoleHeight = 600;
		if(GConfig)
		{
			GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleHeight"), ConsoleHeight, GGameIni);
		}
		
		Frame.origin.y -= (ConsoleHeight - Frame.size.height);
		Frame.size.height = ConsoleHeight;

		[Window setFrame:Frame display:YES animate:YES];
		[LogView setHidden:NO];
	}
	else
	{
		CGFloat Height = [LogView frame].size.height;
		[LogView setHidden:YES];
		NSRect Frame = [Window frame];
		Frame.size.height -= Height;
		Frame.origin.y += Height;
		[Window setFrame:Frame display:YES animate:YES];
	}
}

-(id)init
{
	id obj = [super init];
	if(obj)
	{
		int32 ConsoleWidth = 800;
		int32 ConsoleHeight = 600;
		int32 ConsolePosX = 0;
		int32 ConsolePosY = 0;
		bool bHasX = false;
		bool bHasY = false;
		
		if(GConfig)
		{
			GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleWidth"), ConsoleWidth, GGameIni);
			GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleHeight"), ConsoleHeight, GGameIni);
			bHasX = GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleX"), ConsolePosX, GGameIni);
			bHasY = GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleY"), ConsolePosY, GGameIni);
		}
		
		NSRect WindowRect = NSMakeRect(ConsolePosX, ConsolePosY, ConsoleWidth, ConsoleHeight);
		
		Window = [[NSWindow alloc] initWithContentRect:WindowRect styleMask:NSWindowStyleMaskTitled|NSWindowStyleMaskMiniaturizable|NSWindowStyleMaskResizable|NSWindowStyleMaskClosable backing:NSBackingStoreBuffered defer:NO];
		[Window setTitle:@"Unreal Engine"];
		[Window setReleasedWhenClosed:NO];
		[Window setMinSize:NSMakeSize(400, 100)];
		[Window setRestorable:NO];
		[Window disableSnapshotRestoration];
		
		NSView* View = [Window contentView];
		[View setAutoresizesSubviews:YES];
		[View setAutoresizingMask:NSViewWidthSizable | NSViewHeightSizable];
		
		ShowLogButton = [[NSButton new] autorelease];
		NSRect ShowLogRect = [ShowLogButton frame];
		{
			[ShowLogButton setIdentifier:@"ShowLogButton"];
			[ShowLogButton setButtonType:NSButtonTypeMomentaryPushIn];
			[ShowLogButton setBezelStyle:NSBezelStyleRounded];
			[ShowLogButton setTitle:@"Show Log"];
			[ShowLogButton sizeToFit];
			ShowLogRect = [ShowLogButton frame];
			ShowLogRect.origin.x = WindowRect.size.width - 8 - ShowLogRect.size.width;
			ShowLogRect.origin.y = WindowRect.size.height - ShowLogRect.size.height - 8;
			
			[ShowLogButton setFrameOrigin:ShowLogRect.origin];
			[ShowLogButton setAutoresizingMask:NSViewMinXMargin | NSViewMinYMargin];
			[ShowLogButton setTarget:self];
			[ShowLogButton setAction:@selector(toggleLog)];
		}
		
		CancelButton = [[NSButton new] autorelease];
		NSRect CancelRect = [CancelButton frame];
		{
			[CancelButton setIdentifier:@"CancelButton"];
			[CancelButton setTitle:@"Cancel"];
			[CancelButton setButtonType:NSButtonTypeMomentaryPushIn];
			[CancelButton setBezelStyle:NSBezelStyleRounded];
			[CancelButton sizeToFit];
			CancelRect = [CancelButton frame];
			CancelRect.origin.x = ShowLogRect.origin.x - CancelRect.size.width - 4;
			CancelRect.origin.y = ShowLogRect.origin.y;
			
			[CancelButton setFrameOrigin:CancelRect.origin];
			[CancelButton setAutoresizingMask:NSViewMinXMargin | NSViewMinYMargin];
			[CancelButton setTarget:self];
			[CancelButton setAction:@selector(hideWindow)];
		}
		
		StatusLabel = [[[NSTextField alloc] initWithFrame:NSMakeRect(0, 0, 100, 18)] autorelease];
		NSRect StatusRect = [StatusLabel frame];
		{
			[StatusLabel setBezeled:NO];
			[StatusLabel setDrawsBackground:NO];
			[StatusLabel setFont:[NSFont labelFontOfSize:[NSFont systemFontSize]]];
			[StatusLabel setSelectable:NO];
			[StatusLabel setEditable:NO];
			[StatusLabel setBordered:NO];
			[StatusLabel setStringValue:@"Progress:"];
			StatusRect = [StatusLabel frame];
			
			StatusRect.size.width = CancelRect.origin.x - 16;
			StatusRect.origin.x = 8;
			StatusRect.origin.y = ShowLogRect.origin.y + ((ShowLogRect.size.height - [NSFont systemFontSize]) / 2);
			
			[StatusLabel setIdentifier:@"StatusLabel"];
			[StatusLabel setFrame:StatusRect];
			[StatusLabel setAutoresizingMask:NSViewWidthSizable | NSViewMinYMargin];
		}
		
		ProgressIndicator = [[NSProgressIndicator new] autorelease];
		NSRect ProgressRect = [ProgressIndicator frame];
		{
			[ProgressIndicator setStyle:NSProgressIndicatorStyleBar];
			[ProgressIndicator sizeToFit];
			ProgressRect = [ProgressIndicator frame];
			ProgressRect.size.width = WindowRect.size.width - 16;
			ProgressRect.origin.x = 8;
			ProgressRect.origin.y = CancelRect.origin.y - ProgressRect.size.height - 8;
			
			[ProgressIndicator setIdentifier:@"ProgressIndicator"];
			[ProgressIndicator setIndeterminate:YES];
			[ProgressIndicator setFrame:ProgressRect];
			[ProgressIndicator setAutoresizingMask:NSViewWidthSizable | NSViewMinYMargin];
		}
		
		TextView = [[NSTextView new] autorelease];
		NSRect TextRect = [TextView frame];
		{
			[TextView setIdentifier:@"TextView"];
			[TextView setVerticallyResizable:YES];
			[TextView setHorizontallyResizable:NO];
			[TextView setBackgroundColor: [NSColor blackColor]];
			[TextView setMinSize:NSMakeSize( 0.0, 0.0 ) ];
			[TextView setMaxSize:NSMakeSize( FLT_MAX, FLT_MAX )];
			TextRect = NSMakeRect(8, 8, WindowRect.size.width - 16, ProgressRect.origin.y - 16);
			
			[TextView setAutoresizingMask:NSViewWidthSizable | NSViewHeightSizable];
			
			LogView = [[NSScrollView new] autorelease];
			[LogView setHasVerticalScroller:YES];
			[LogView setHasHorizontalScroller:NO];
			[LogView setAutohidesScrollers:YES];
			[LogView setAutoresizesSubviews:YES];
			[LogView setAutoresizingMask:NSViewWidthSizable | NSViewHeightSizable | NSViewMaxYMargin];
			[LogView setFrame:TextRect];
			
			[TextView setFrameSize:[LogView contentSize]];
			[[TextView textContainer] setContainerSize:NSMakeSize( [LogView contentSize].width, FLT_MAX )];
			[[TextView textContainer] setWidthTracksTextView:YES];
			
			[LogView setDocumentView:TextView];
		}
		
		[View addSubview:ShowLogButton];
		[View addSubview:StatusLabel];
		[View addSubview:ProgressIndicator];
		[View addSubview:CancelButton];
		[View addSubview:LogView];
		
		[ProgressIndicator startAnimation:nil];
		
		if (!bHasX || !bHasY)
		{
			[Window center];
		}
	}
	return self;
}

-(void)dealloc
{
	[Window close];
	[Window release];
	[super dealloc];
}

-(void)setShowProgress:(bool)bShowProgress
{
	if(bShowProgress)
	{
		[ProgressIndicator stopAnimation:nil];
		[ProgressIndicator setIndeterminate:NO];
	}
	else
	{
		if(![LogView isHidden])
		{
			[self toggleLog];
		}
		[ProgressIndicator setIndeterminate:YES];
		[ProgressIndicator startAnimation:nil];
	}
}

-(void)setShowCancelButton:(bool)bShowCancelButton
{
	BOOL bHideCancel = !bShowCancelButton;
	[CancelButton setHidden:bHideCancel];
}

-(void)setTitleText:(NSString*)Title
{
	[Window setTitle:Title];
}

-(void)setStatusText:(NSString*)Text
{
	[StatusLabel setStringValue:Text];
}

-(void)setProgress:(double)Progress total:(double)Total
{
	if(![ProgressIndicator isIndeterminate])
	{
		[ProgressIndicator setMaxValue:Total];
		[ProgressIndicator setMinValue:0.0];
		[ProgressIndicator setDoubleValue:Progress];
	}
}

-(void)showWindow
{
	[Window makeKeyAndOrderFront:nil];
}

-(void)hideWindow
{
	[Window orderOut:nil];
}

-(bool)windowOpen
{
	return [Window isVisible];
}
@end

FMacNativeFeedbackContext::FMacNativeFeedbackContext()
	: FFeedbackContext()
	, WindowController(nil)
	, Context( NULL )
	, OutstandingTasks(0)
	, SlowTaskCount( 0 )
	, bShowingConsoleForSlowTask( false )
{
	MainThreadCall(^{
		WindowController = [[FMacNativeFeedbackContextWindowController alloc] init];
	}, true, UnrealNilEventMode);
	SetDefaultTextColor();
}

FMacNativeFeedbackContext::~FMacNativeFeedbackContext()
{
	do
	{
		FPlatformApplicationMisc::PumpMessages( true );
	} while(OutstandingTasks);
	
	MainThreadCall(^{
		[WindowController release];
	}, true, UnrealNilEventMode);
}

void FMacNativeFeedbackContext::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Serialize(Data, Verbosity, Category, -1.0);
}

void FMacNativeFeedbackContext::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	FFeedbackContext::Serialize(Data, Verbosity, Category, Time);
	SerializeToWindow(Data, Verbosity, Category, Time);
}

void FMacNativeFeedbackContext::SerializeRecord(const UE::FLogRecord& Record)
{
	FFeedbackContext::SerializeRecord(Record);

	TStringBuilder<512> Text;
	Record.FormatMessageTo(Text);
	SerializeToWindow(*Text, Record.GetVerbosity(), Record.GetCategory(), -1.0);
}

void FMacNativeFeedbackContext::SerializeToWindow(const TCHAR* Data, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	if (WindowController && bShowingConsoleForSlowTask)
	{
		FScopeLock ScopeLock(&CriticalSection);

		static bool Entry=0;
		if( !GIsCriticalError || Entry )
		{
			// here we can change the color of the text to display, it's in the format:
			// ForegroundRed | ForegroundGreen | ForegroundBlue | ForegroundBright | BackgroundRed | BackgroundGreen | BackgroundBlue | BackgroundBright
			// where each value is either 0 or 1 (can leave off trailing 0's), so
			// blue on bright yellow is "00101101" and red on black is "1"
			// An empty string reverts to the normal gray on black

			if (Verbosity == ELogVerbosity::SetColor)
			{
				if (FCString::Stricmp(Data, TEXT("")) == 0)
				{
					SetDefaultTextColor();
				}
				else
				{
					SCOPED_AUTORELEASE_POOL;

					// turn the string into a bunch of 0's and 1's
					TCHAR String[9];
					FMemory::Memset(String, 0, sizeof(TCHAR) * UE_ARRAY_COUNT(String));
					FCString::Strncpy(String, Data, UE_ARRAY_COUNT(String));
					for (TCHAR* S = String; *S; S++)
					{
						*S -= '0';
					}

					NSMutableArray* Colors = [[NSMutableArray alloc] init];
					NSMutableArray* AttributeKeys = [[NSMutableArray alloc] init];

					// Get FOREGROUND_INTENSITY and calculate final color
					CGFloat Intensity = String[3] ? 1.0 : 0.5;
					[Colors addObject:[NSColor colorWithSRGBRed:(String[0] ? 1.0 * Intensity : 0.0) green:(String[1] ? 1.0 * Intensity : 0.0) blue:(String[2] ? 1.0 * Intensity : 0.0) alpha:1.0]];
					
					// Get BACKGROUND_INTENSITY and calculate final color
					Intensity = String[7] ? 1.0 : 0.5;
					[Colors addObject:[NSColor colorWithSRGBRed:(String[4] ? 1.0 * Intensity : 0.0) green:(String[5] ? 1.0 * Intensity : 0.0) blue:(String[6] ? 1.0 * Intensity : 0.0) alpha:1.0]];

					[AttributeKeys addObject:NSForegroundColorAttributeName];
					[AttributeKeys addObject:NSBackgroundColorAttributeName];

					OutstandingTasks++;
					MainThreadCall(^{
						if( TextViewTextColor )
							[TextViewTextColor release];

						TextViewTextColor = [[NSDictionary alloc] initWithObjects:Colors forKeys:AttributeKeys];

						[Colors release];
						[AttributeKeys release];
						OutstandingTasks--;
					}, false);
				}
			}
			else
			{
				SCOPED_AUTORELEASE_POOL;

				TCHAR OutputString[MAX_SPRINTF]=TEXT(""); //@warning: this is safe as FCString::Sprintf only use 1024 characters max
				FCString::Sprintf(OutputString,TEXT("%s%s"), *FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Data, GPrintLogTimes, Time), LINE_TERMINATOR);

				CFStringRef CocoaText = FPlatformString::TCHARToCFString(OutputString);

				OutstandingTasks++;
				MainThreadCall(^{
					NSAttributedString *AttributedString = [[NSAttributedString alloc] initWithString:(NSString*)CocoaText attributes:TextViewTextColor];
					[[WindowController->TextView textStorage] appendAttributedString:AttributedString];
					[WindowController->TextView scrollRangeToVisible:NSMakeRange([[WindowController->TextView string] length], 0)];
					[AttributedString release];
					CFRelease(CocoaText);
					OutstandingTasks--;
				}, false);

				if(!MacApplication)
				{
					FPlatformApplicationMisc::PumpMessages( true );
				}
			}
		}
		else
		{
			Entry=1;
			try
			{
				// Ignore errors to prevent infinite-recursive exception reporting.
				SerializeToWindow(Data, Verbosity, Category, Time);
			}
			catch( ... )
			{}
			Entry=0;
		}
	}
}

bool FMacNativeFeedbackContext::YesNof(const FText& Text)
{
	return GWarn->YesNof(Text);
}

bool FMacNativeFeedbackContext::ReceivedUserCancel()
{
	bool bReceivedUserCancel = false;
	if(WindowController != NULL && bShowingConsoleForSlowTask && ![WindowController windowOpen])
	{
		bReceivedUserCancel = true;
	}
	return bReceivedUserCancel;
}

void FMacNativeFeedbackContext::StartSlowTask(const FText& Task, bool bInShowCancelButton)
{
	FFeedbackContext::StartSlowTask(Task, bInShowCancelButton);

	if(WindowController != NULL && !bShowingConsoleForSlowTask)
	{
		MainThreadCall(^{
			[WindowController setTitleText:Task.ToString().GetNSString()];
			[WindowController setStatusText:@"Progress:"];
			[WindowController setShowCancelButton:bInShowCancelButton];
			[WindowController setShowProgress:true];
			[WindowController setProgress:0 total:1];
			[WindowController setShowProgress:false];
			
			SetDefaultTextColor();
			
			[WindowController showWindow];
			
			bShowingConsoleForSlowTask = true;
		});
	}
}

void FMacNativeFeedbackContext::FinalizeSlowTask()
{
	FFeedbackContext::FinalizeSlowTask();

	if(bShowingConsoleForSlowTask)
	{
		MainThreadCall(^{
			if(WindowController != NULL)
			{
				[WindowController hideWindow];
			}
			bShowingConsoleForSlowTask = false;
		});
	}
}

void FMacNativeFeedbackContext::ProgressReported( const float TotalProgressInterp, FText DisplayMessage )
{
	if(WindowController != NULL && bShowingConsoleForSlowTask)
	{
		MainThreadCall(^{
			[WindowController setStatusText:DisplayMessage.ToString().GetNSString()];
			[WindowController setShowProgress:true];
			[WindowController setProgress:TotalProgressInterp total:1];
		});
	}
}

FContextSupplier* FMacNativeFeedbackContext::GetContext() const
{
	return Context;
}

void FMacNativeFeedbackContext::SetContext( FContextSupplier* InSupplier )
{
	Context = InSupplier;
}

void FMacNativeFeedbackContext::SetDefaultTextColor()
{
	SCOPED_AUTORELEASE_POOL;
	FScopeLock ScopeLock( &CriticalSection );
	
	NSMutableArray* Colors = [[NSMutableArray alloc] init];
	NSMutableArray* AttributeKeys = [[NSMutableArray alloc] init];
	
	[Colors addObject:[NSColor grayColor]];
	[Colors addObject:[NSColor blackColor]];
	
	[AttributeKeys addObject:NSForegroundColorAttributeName];
	[AttributeKeys addObject:NSBackgroundColorAttributeName];
	
	OutstandingTasks++;
	MainThreadCall(^{
		if( TextViewTextColor )
			[TextViewTextColor release];
		
		TextViewTextColor = [[NSDictionary alloc] initWithObjects:Colors forKeys:AttributeKeys];
		
		[Colors release];
		[AttributeKeys release];
		OutstandingTasks--;
	}, false);
}
