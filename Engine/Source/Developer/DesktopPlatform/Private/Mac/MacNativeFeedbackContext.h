// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Mac/MacSystemIncludes.h"
#include "Misc/FeedbackContext.h"

@interface FMacNativeFeedbackContextWindowController : NSObject
{
@public
	NSTextView* TextView;
@private
	NSScrollView* LogView;
	NSWindow* Window;
	NSTextField* StatusLabel;
	NSButton* CancelButton;
	NSButton* ShowLogButton;
	NSProgressIndicator* ProgressIndicator;
}
-(id)init;
-(void)dealloc;
-(void)setShowProgress:(bool)bShowProgress;
-(void)setShowCancelButton:(bool)bShowCancelButton;
-(void)setTitleText:(NSString*)Title;
-(void)setStatusText:(NSString*)Text;
-(void)setProgress:(double)Progress total:(double)Total;
-(void)showWindow;
-(void)hideWindow;
-(bool)windowOpen;
@end

/**
 * Feedback context implementation for Mac.
 */
class FMacNativeFeedbackContext : public FFeedbackContext
{
public:
	// Constructor.
	FMacNativeFeedbackContext();
	virtual ~FMacNativeFeedbackContext();

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;
	virtual void SerializeRecord(const UE::FLogRecord& Record) override;

	virtual bool YesNof(const FText& Text) override;

	virtual bool ReceivedUserCancel() override;

	virtual void StartSlowTask( const FText& Task, bool bShowCancelButton=false ) override;
	virtual void FinalizeSlowTask( ) override;
	virtual void ProgressReported( const float TotalProgressInterp, FText DisplayMessage ) override;

	FContextSupplier* GetContext() const override;
	void SetContext(FContextSupplier* InContext) override;

private:
	void SerializeToWindow(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time);

	void SetDefaultTextColor();
	
private:
	/** Critical section for Serialize() */
	FCriticalSection CriticalSection;
	FMacNativeFeedbackContextWindowController* WindowController;
	NSDictionary* TextViewTextColor;
	FContextSupplier* Context;
	uint64 OutstandingTasks;
	int32 SlowTaskCount;
	bool bShowingConsoleForSlowTask;
};
