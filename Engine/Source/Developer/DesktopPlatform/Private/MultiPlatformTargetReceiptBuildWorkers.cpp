// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiPlatformTargetReceiptBuildWorkers.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"

FMultiPlatformTargetReceiptBuildWorkers::FMultiPlatformTargetReceiptBuildWorkers(FStringView TargetReceiptFilePath)
{
	FString PathString(TargetReceiptFilePath);
	if (PathString.Contains(TEXT("$(Platform)")))
	{
		bAllPlatformsInitialized = true;
		PlatformSpecificWorkerFactories[SupportedPlatform_Win64].EmplaceUnchecked(PathString.Replace(TEXT("$(Platform)"), TEXT("Win64")));
		PlatformSpecificWorkerFactories[SupportedPlatform_Mac].EmplaceUnchecked(PathString.Replace(TEXT("$(Platform)"), TEXT("Mac")));
		PlatformSpecificWorkerFactories[SupportedPlatform_Linux].EmplaceUnchecked(PathString.Replace(TEXT("$(Platform)"), TEXT("Linux")));
	}
	else
	{
		PlatformSpecificWorkerFactories[0].EmplaceUnchecked(PathString);
	}
}

FMultiPlatformTargetReceiptBuildWorkers::~FMultiPlatformTargetReceiptBuildWorkers()
{
	if (bAllPlatformsInitialized)
	{
		PlatformSpecificWorkerFactories[SupportedPlatform_Win64].DestroyUnchecked();
		PlatformSpecificWorkerFactories[SupportedPlatform_Mac].DestroyUnchecked();
		PlatformSpecificWorkerFactories[SupportedPlatform_Linux].DestroyUnchecked();
	}
	else
	{
		PlatformSpecificWorkerFactories[0].DestroyUnchecked();
	}
}
