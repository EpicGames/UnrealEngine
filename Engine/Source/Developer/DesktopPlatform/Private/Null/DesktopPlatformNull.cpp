// Copyright Epic Games, Inc. All Rights Reserved.

#include "Null/DesktopPlatformNull.h"

#include "DesktopPlatformPrivate.h"
#include "HAL/FileManager.h"
#include "HAL/ThreadHeartBeat.h"
#include "Internationalization/Regex.h"
#include "ISlateFileDialogModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FeedbackContextMarkup.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#if PLATFORM_LINUX
#include "ISlateFontDialogModule.h" 
#endif

#define LOCTEXT_NAMESPACE "DesktopPlatform"
#define MAX_FILETYPES_STR 4096
#define MAX_FILENAME_STR 65536

DEFINE_LOG_CATEGORY_STATIC(LogDesktopPlatformNull, Log, All);

FDesktopPlatformNull::FDesktopPlatformNull()
	:	FDesktopPlatformBase()
{
}

FDesktopPlatformNull::~FDesktopPlatformNull()
{
}

bool FDesktopPlatformNull::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->OpenFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, OutFilterIndex);
	}

	return false;
}

bool FDesktopPlatformNull::OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->OpenFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames);
	}

	return false;
}

bool FDesktopPlatformNull::SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->SaveFileDialog(ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames);
	}

	return false;
}

bool FDesktopPlatformNull::OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, FString& OutFolderName)
{
	if (!FModuleManager::Get().IsModuleLoaded("SlateFileDialogs"))
	{
		FModuleManager::Get().LoadModule("SlateFileDialogs");
	}

	ISlateFileDialogsModule *FileDialog = FModuleManager::GetModulePtr<ISlateFileDialogsModule>("SlateFileDialogs");

	if (FileDialog)
	{
		return FileDialog->OpenDirectoryDialog(ParentWindowHandle, DialogTitle, DefaultPath, OutFolderName);
	}

	return false;
}

bool FDesktopPlatformNull::OpenFontDialog(const void* ParentWindowHandle, FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags)
{
#if PLATFORM_LINUX
	if (!FModuleManager::Get().IsModuleLoaded("SlateFontDialog"))
	{
		FModuleManager::Get().LoadModule("SlateFontDialog");
	}

	ISlateFontDialogModule* FontDialog = FModuleManager::GetModulePtr<ISlateFontDialogModule>("SlateFontDialog");
	
	if (FontDialog)
	{
		return FontDialog->OpenFontDialog(OutFontName, OutHeight, OutFlags);
	}
	else
	{
		UE_LOGF(LogDesktopPlatformNull, Warning, "Error reading results of font dialog");
	}
#endif
	return false;
}

bool FDesktopPlatformNull::FileDialogShared(bool bSave, const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex)
{
	return false;
}

bool FDesktopPlatformNull::RegisterEngineInstallation(const FString &RootDir, FString &OutIdentifier)
{
	bool bRes = false;
	if (IsValidRootDirectory(RootDir))
	{
		FConfigFile ConfigFile;
		FString ConfigPath = FString(FPlatformProcess::ApplicationSettingsDir()) / FString(TEXT("UnrealEngine")) / FString(TEXT("Install.ini"));
		ConfigFile.Read(ConfigPath);

		// If this is an installed build, use that Guid instead of generating a new one
		FString InstallationIdPath = FString(RootDir / "Engine" / "Build" / "InstalledBuild.txt");
		FArchive* File = IFileManager::Get().CreateFileReader(*InstallationIdPath, FILEREAD_Silent);
		if(File)
		{
			FFileHelper::LoadFileToString(OutIdentifier, *File);
			FGuid GuidCheck(OutIdentifier);
			if(!GuidCheck.IsValid())
			{
				OutIdentifier = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens);
			}

			File->Close();
			delete File;
		}
		else
		{
			OutIdentifier = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens);
		}
		
		ConfigFile.AddUniqueToSection(TEXT("Installations"), *OutIdentifier, RootDir);
		ConfigFile.Write(ConfigPath);
		bRes = true;
	}
	return bRes;
}

void FDesktopPlatformNull::EnumerateEngineInstallations(TMap<FString, FString> &OutInstallations)
{
	EnumerateLauncherEngineInstallations(OutInstallations);
	// TODO (William.Belcher): See if we actually need to run this when using the null platform
}

bool FDesktopPlatformNull::IsSourceDistribution(const FString &RootDir)
{
	// Use the default test
	return FDesktopPlatformBase::IsSourceDistribution(RootDir);
}

bool FDesktopPlatformNull::VerifyFileAssociations()
{
	return true;
}

bool FDesktopPlatformNull::UpdateFileAssociations()
{
	return true;
}

bool FDesktopPlatformNull::OpenProject(const FString &ProjectFileName)
{
	// Get the project filename in a native format
	FString PlatformProjectFileName = ProjectFileName;
	FPaths::MakePlatformFilename(PlatformProjectFileName);

	STUBBED("FDesktopPlatformNull::OpenProject");
	return false;
}

bool FDesktopPlatformNull::RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutExitCode)
{
	OutExitCode = 1;

	// Get the path to UBT
	FString UnrealBuildToolPath = GetUnrealBuildToolExecutableFilename(RootDir);
	if(IFileManager::Get().FileSize(*UnrealBuildToolPath) < 0)
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("Couldn't find UnrealBuildTool at '%s'"), *UnrealBuildToolPath);
		return false;
	}

	// Write the output
	Warn->Logf(TEXT("Running %s %s"), *UnrealBuildToolPath, *Arguments);

	return FFeedbackContextMarkup::PipeProcessOutput(Description, UnrealBuildToolPath, Arguments, Warn, &OutExitCode) && OutExitCode == 0;
}

FFeedbackContext* FDesktopPlatformNull::GetNativeFeedbackContext()
{
	//unimplemented();
	STUBBED("FDesktopPlatformNull::GetNativeFeedbackContext");
	return GWarn;
}

FString FDesktopPlatformNull::GetUserTempPath()
{
	return FString(FPlatformProcess::UserTempDir());
}

FString FDesktopPlatformNull::GetOidcTokenExecutableFilename(const FString& RootDir) const
{	
	return "";
}

#undef LOCTEXT_NAMESPACE
