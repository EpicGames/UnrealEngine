// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformInfo.h"
#include "DesktopPlatformPrivate.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/CommandLine.h"

#define LOCTEXT_NAMESPACE "PlatformInfo"

#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA

namespace PlatformInfo
{
	TArray<FName> AllPlatformGroupNames;
	TArray<FName> AllVanillaPlatformNames;
	TArray<FPreviewPlatformMenuItem> PreviewPlatformMenuItems;

	namespace
	{

		TArray<FTargetPlatformInfo*> AllPlatformInfoArray;
		TArray<FTargetPlatformInfo*> VanillaPlatformInfoArray;

	}

FTargetPlatformInfo::FTargetPlatformInfo(const FString& InIniPlatformName, EBuildTargetType InType, const FString& InCookFlavor)
{
	IniPlatformName = *InIniPlatformName;
	PlatformType = InType;

	// calculate the name of the TargetPlatform
	FString TPName = InIniPlatformName;

	// by default we are vanilla, so point to ourself
	VanillaInfo = this;
	if (InCookFlavor != TEXT("") || PlatformType != EBuildTargetType::Game)
	{
		// if we are non-Game or cook flavor, then we need to find the most base version (will have same name as IniPlatform)
		FTargetPlatformInfo** FoundInfo = AllPlatformInfoArray.FindByPredicate([this](const FTargetPlatformInfo* Item) -> bool
 		{
 			return Item->Name == IniPlatformName;
 		});

		checkf(FoundInfo != nullptr, TEXT("Creating a TargetPlatform (%s, %s, %s) that needed a 'vanilla' TP already created, but it wasn't found. Create Game TPs first, and all cook flavors last"), *InIniPlatformName, LexToString(InType), *InCookFlavor);

		VanillaInfo = *FoundInfo;
	}

	FString DisplayString = InIniPlatformName;

	// handle cook flavors
	PlatformFlags = EPlatformFlags::None;
	if (InCookFlavor != TEXT(""))
	{
		// mark us as a cookflavor
		PlatformFlags = EPlatformFlags::CookFlavor;
		PlatformFlavor = *InCookFlavor;

		// append flavor with _
		TPName += FString::Printf(TEXT("_%s"), *InCookFlavor);
		
		// put the flavor in parens
		DisplayString += FString::Printf(TEXT(" (%s)"), *InCookFlavor);

		// append UAT commandline with cook flavor
		UATCommandLine += FString::Printf(TEXT(" -cookflavor=%s"), *InCookFlavor);

		VanillaInfo->Flavors.AddUnique(this);
	}

	// now append the build type (game type has no type suffix, all others do)
	if (PlatformType != EBuildTargetType::Game)
	{
		TPName += LexToString(PlatformType);

		// put the type in parens
		DisplayString += FString::Printf(TEXT(" (%s)"), LexToString(PlatformType));

		// client and server builds need to modify the commandline
		if (PlatformType == EBuildTargetType::Client)
		{
			UATCommandLine += TEXT(" -client");
		}
		else if (PlatformType == EBuildTargetType::Server)
		{
			UATCommandLine += TEXT(" -server -noclient");
		}

		VanillaInfo->Flavors.AddUnique(this);
	}

	// now we can store the final values
	Name = *TPName;
	DisplayName = FText::FromString(DisplayString);
 	DataDrivenPlatformInfo = &FDataDrivenPlatformInfoRegistry::GetPlatformInfo(InIniPlatformName);

	// update various arrays
	if (DataDrivenPlatformInfo->PlatformGroupName != NAME_None)
	{
		AllPlatformGroupNames.AddUnique(DataDrivenPlatformInfo->PlatformGroupName);
	}

	AllPlatformInfoArray.Add(this);
	if (VanillaInfo == this)
	{
		VanillaPlatformInfoArray.Add(this);
		AllVanillaPlatformNames.AddUnique(Name);
	}
}


const FTargetPlatformInfo* FindPlatformInfo(const FName& InPlatformName)
{
	checkf(AllPlatformInfoArray.Num() > 0, TEXT("Querying for TargetPlatformInfo objects before they are ready!"));

	for(const FTargetPlatformInfo* PlatformInfo : AllPlatformInfoArray)
	{
		if(PlatformInfo->Name == InPlatformName)
		{
			return PlatformInfo;
		}
	}

	return nullptr;
}

const FTargetPlatformInfo* FindVanillaPlatformInfo(const FName& InPlatformName)
{
	checkf(AllPlatformInfoArray.Num() > 0, TEXT("Querying for TargetPlatformInfo objects before they are ready!"));

	const FTargetPlatformInfo* const FoundInfo = FindPlatformInfo(InPlatformName);
	return FoundInfo ? FoundInfo->VanillaInfo : nullptr;
}

void UpdatePlatformDisplayName(FString InPlatformName, FText InDisplayName)
{
	checkf(AllPlatformInfoArray.Num() > 0, TEXT("Querying for TargetPlatformInfo objects before they are ready!"));

	for (FTargetPlatformInfo* PlatformInfo : AllPlatformInfoArray)
	{
		if (PlatformInfo->Name == FName(*InPlatformName))
		{
			PlatformInfo->DisplayName = InDisplayName;
		}
	}
}

const TArray<FTargetPlatformInfo*>& GetPlatformInfoArray()
{
	checkf(AllPlatformInfoArray.Num() > 0, TEXT("Querying for TargetPlatformInfo objects before they are ready!"));

	return AllPlatformInfoArray;
}

const TArray<FTargetPlatformInfo*>& GetVanillaPlatformInfoArray()
{
	checkf(AllPlatformInfoArray.Num() > 0, TEXT("Querying for TargetPlatformInfo objects before they are ready!"));

	return VanillaPlatformInfoArray;
}

const TArray<FName>& GetAllPlatformGroupNames()
{
	checkf(AllPlatformInfoArray.Num() > 0, TEXT("Querying for TargetPlatformInfo objects before they are ready!"));

	return PlatformInfo::AllPlatformGroupNames;
}

const TArray<FName>& GetAllVanillaPlatformNames()
{
	checkf(AllPlatformInfoArray.Num() > 0, TEXT("Querying for TargetPlatformInfo objects before they are ready!"));

	return PlatformInfo::AllVanillaPlatformNames;
}

} // namespace PlatformInfo

#endif


#undef LOCTEXT_NAMESPACE
