// Copyright Epic Games, Inc. All Rights Reserved.

#include "TargetReceipt.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"

bool TryParseBuildVersion(const FJsonObject& Object, FBuildVersion& OutVersion)
{
	if (Object.TryGetNumberField(TEXT("MajorVersion"), OutVersion.MajorVersion) && Object.TryGetNumberField(TEXT("MinorVersion"), OutVersion.MinorVersion) && Object.TryGetNumberField(TEXT("PatchVersion"), OutVersion.PatchVersion))
	{
		Object.TryGetNumberField(TEXT("Changelist"), OutVersion.Changelist);
		Object.TryGetNumberField(TEXT("CompatibleChangelist"), OutVersion.CompatibleChangelist);
		Object.TryGetNumberField(TEXT("IsLicenseeVersion"), OutVersion.IsLicenseeVersion);
		Object.TryGetNumberField(TEXT("IsPromotedBuild"), OutVersion.IsPromotedBuild);
		Object.TryGetStringField(TEXT("BranchName"), OutVersion.BranchName);
		Object.TryGetStringField(TEXT("BuildId"), OutVersion.BuildId);
		Object.TryGetStringField(TEXT("BuildVersion"), OutVersion.BuildVersion);
		return true;
	}
	return false;
}

bool FTargetReceipt::Read(const FString& FileName, bool bExpandVariables)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTargetReceipt::Read);

	// Read the file from disk
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *FileName))
	{
		return false;
	}

	TSharedPtr<FJsonObject> Object;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContents);
	if (!FJsonSerializer::Deserialize(Reader, Object) || !Object.IsValid())
	{
		return false;
	}

	// Get the project file
	FString RelativeProjectFile;
	if(Object->TryGetStringField(TEXT("Project"), RelativeProjectFile))
	{
		ProjectFile = FPaths::Combine(FPaths::GetPath(FileName), RelativeProjectFile);
		FPaths::MakeStandardFilename(ProjectFile);
		ProjectDir = FPaths::GetPath(ProjectFile);
	}

	// Get the target name
	if (!Object->TryGetStringField(TEXT("TargetName"), TargetName))
	{
		return false;
	}
	if (!Object->TryGetStringField(TEXT("Platform"), Platform))
	{
		return false;
	}
	if (!Object->TryGetStringField(TEXT("Architecture"), Architecture))
	{
		return false;
	}
	if (!Object->TryGetStringField(TEXT("BuildSettingsVersion"), BuildSettingsVersion))
	{
		// This was added at 5.7; no-op on failure.
	}
	if (!Object->TryGetStringField(TEXT("TargetBuildEnvironment"), TargetBuildEnvironment))
	{
		// This was added at 5.7; no-op on failure.
	}

	// Read the configuration
	FString ConfigurationString;
	if (!Object->TryGetStringField(TEXT("Configuration"), ConfigurationString) || !LexTryParseString(Configuration, *ConfigurationString))
	{
		return false;
	}

	// Read the target type
	FString TargetTypeString;
	if (!Object->TryGetStringField(TEXT("TargetType"), TargetTypeString) || !LexTryParseString(TargetType, *TargetTypeString))
	{
		return false;
	}

	// Read the version information
	const TSharedPtr<FJsonObject>* VersionObject;
	if (!Object->TryGetObjectField(TEXT("Version"), VersionObject) || !VersionObject->IsValid() || !TryParseBuildVersion(*VersionObject->Get(), Version))
	{
		return false;
	}

	// Get the launch path(s). LaunchCmd is optional.
	if (FString MaybeLaunchCmd; Object->TryGetStringField(TEXT("LaunchCmd"), MaybeLaunchCmd))
	{
		LaunchCmd = MoveTemp(MaybeLaunchCmd);
	}

	if (!Object->TryGetStringField(TEXT("Launch"), Launch))
	{
		// If the regular launch field is not present try using the LaunchCmd field
		if (!LaunchCmd.IsSet())
		{
			return false;
		}
		else
		{
			Launch = LaunchCmd.GetValue();
		}
	}

	if (bExpandVariables)
	{
		ExpandVariables(Launch);

		if (LaunchCmd.IsSet())
		{
			ExpandVariables(LaunchCmd.GetValue());
		}
	}

	// Read the list of build products
	const TArray<TSharedPtr<FJsonValue>>* BuildProductsArray;
	if (Object->TryGetArrayField(TEXT("BuildProducts"), BuildProductsArray))
	{
		for(const TSharedPtr<FJsonValue>& BuildProductValue : *BuildProductsArray)
		{
			const TSharedPtr<FJsonObject>* BuildProductObject;
			if(!BuildProductValue->TryGetObject(BuildProductObject))
			{
				return false;
			}

			FBuildProduct BuildProduct;
			if(!(*BuildProductObject)->TryGetStringField(TEXT("Type"), BuildProduct.Type) || !(*BuildProductObject)->TryGetStringField(TEXT("Path"), BuildProduct.Path))
			{
				return false;
			}

			if (bExpandVariables)
			{
				ExpandVariables(BuildProduct.Path);
			}

			BuildProducts.Add(MoveTemp(BuildProduct));
		}
	}

	// Read the list of runtime dependencies
	const TArray<TSharedPtr<FJsonValue>>* RuntimeDependenciesArray;
	if (Object->TryGetArrayField(TEXT("RuntimeDependencies"), RuntimeDependenciesArray))
	{
		for(const TSharedPtr<FJsonValue>& RuntimeDependencyValue : *RuntimeDependenciesArray)
		{
			const TSharedPtr<FJsonObject>* RuntimeDependencyObject;
			if(!RuntimeDependencyValue->TryGetObject(RuntimeDependencyObject))
			{
				return false;
			}

			FRuntimeDependency RuntimeDependency;
			if(!(*RuntimeDependencyObject)->TryGetStringField(TEXT("Path"), RuntimeDependency.Path) || !(*RuntimeDependencyObject)->TryGetStringField(TEXT("Type"), RuntimeDependency.Type))
			{
				return false;
			}

			if (bExpandVariables)
			{
				ExpandVariables(RuntimeDependency.Path);
			}

			RuntimeDependencies.Add(MoveTemp(RuntimeDependency));
		}
	}

	// Read the list of rules-enabled and disabled plugins
	const TArray<TSharedPtr<FJsonValue>>* PluginsArray;
	if (Object->TryGetArrayField(TEXT("Plugins"), PluginsArray))
	{
		for (const TSharedPtr<FJsonValue>& PluginValue : *PluginsArray)
		{
			const TSharedPtr<FJsonObject>* PluginObject;
			if (!PluginValue->TryGetObject(PluginObject))
			{
				return false;
			}

			FString PluginName;
			bool bPluginEnabled;
			if (!(*PluginObject)->TryGetStringField(TEXT("Name"), PluginName) || !(*PluginObject)->TryGetBoolField(TEXT("Enabled"), bPluginEnabled))
			{
				return false;
			}

			PluginNameToEnabledState.Add(PluginName, bPluginEnabled);
		}
	}

	// Read the list of build plugins
	FJsonSerializableArray BuildPluginsArray;
	if (Object->TryGetStringArrayField(TEXT("BuildPlugins"), BuildPluginsArray))
	{
		BuildPlugins.Append(BuildPluginsArray.GetData(), BuildPluginsArray.Num());
	}	

	// Read the list of additional properties
	const TArray<TSharedPtr<FJsonValue>>* AdditionalPropertiesArray;
	if (Object->TryGetArrayField(TEXT("AdditionalProperties"), AdditionalPropertiesArray))
	{
		for(const TSharedPtr<FJsonValue>& AdditionalPropertyValue : *AdditionalPropertiesArray)
		{
			const TSharedPtr<FJsonObject>* AdditionalPropertyObject;
			if(!AdditionalPropertyValue->TryGetObject(AdditionalPropertyObject))
			{
				return false;
			}

			FReceiptProperty Property;
			if(!(*AdditionalPropertyObject)->TryGetStringField(TEXT("Name"), Property.Name) || !(*AdditionalPropertyObject)->TryGetStringField(TEXT("Value"), Property.Value))
			{
				return false;
			}

			if (bExpandVariables)
			{
				ExpandVariables(Property.Value);
			}

			AdditionalProperties.Add(MoveTemp(Property));
		}
	}

	return true;
}

bool FTargetReceipt::LaunchesCurrentExecutable(bool bCheckLaunchField, bool bCheckLaunchCmdField) const
{
	const FString CurrentExecutable = FPlatformProcess::ExecutablePath();

	if (bCheckLaunchField && FPaths::IsSamePath(Launch, CurrentExecutable))
	{
		return true;
	}

	if (bCheckLaunchCmdField && LaunchCmd.IsSet() && FPaths::IsSamePath(LaunchCmd.GetValue(), CurrentExecutable))
	{
		return true;
	}

	return false;
}

FString FTargetReceipt::GetDefaultPath(const TCHAR* BaseDir, const TCHAR* TargetName, const TCHAR* Platform, EBuildConfiguration Configuration, const TCHAR* BuildArchitecture)
{
	const TCHAR* ArchitectureSuffix = TEXT("");
	if (BuildArchitecture != nullptr)
	{
		ArchitectureSuffix = BuildArchitecture;
	}

	if ((BuildArchitecture == nullptr || BuildArchitecture[0] == 0) && Configuration == EBuildConfiguration::Development)
	{
		return FPaths::Combine(BaseDir, FString::Printf(TEXT("Binaries/%s/%s.target"), Platform, TargetName));
	}
	else
	{
		return FPaths::Combine(BaseDir, FString::Printf(TEXT("Binaries/%s/%s-%s-%s%s.target"), Platform, TargetName, Platform, LexToString(Configuration), ArchitectureSuffix));
	}
}

void FTargetReceipt::ExpandVariables(FString& Path)
{
	static FString EngineDirPrefix = TEXT("$(EngineDir)");
	static FString ProjectDirPrefix = TEXT("$(ProjectDir)");
	if(Path.StartsWith(EngineDirPrefix))
	{
		FString EngineDir = FPaths::EngineDir();
		if (EngineDir.Len() > 0 && EngineDir[EngineDir.Len() - 1] == '/')
		{
			EngineDir = EngineDir.Left(EngineDir.Len() - 1);
		}
		Path = EngineDir + Path.Mid(EngineDirPrefix.Len());
	}
	else if(Path.StartsWith(ProjectDirPrefix) && ProjectDir.Len() > 0)
	{
		Path = ProjectDir + Path.Mid(ProjectDirPrefix.Len());
	}
}
