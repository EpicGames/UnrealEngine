// Copyright Epic Games, Inc. All Rights Reserved.

#include "TargetReceiptBuildWorker.h"

#include "Algo/Accumulate.h"
#include "Algo/MaxElement.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Async/ParallelFor.h"
#include "Compression/CompressedBuffer.h"
#include "Compression/OodleDataCompression.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SharedString.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "DerivedDataBuildWorker.h"
#include "DerivedDataBuildWorkerRegistry.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "IO/IoHash.h"
#include "Logging/StructuredLog.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "String/ParseTokens.h"
#include "TargetReceipt.h"
#include "Tasks/Task.h"
#include "Templates/Projection.h"
#include "Templates/Tuple.h"

#include <atomic>

DEFINE_LOG_CATEGORY_STATIC(LogTargetReceiptBuildWorker, Log, All);

class FTargetReceiptBuildWorkerBuilder
{
	// This needs to match the version exported in DerivedDataBuildWorker.Build.cs.
	inline static const FGuid CurrentWorkerReceiptVersion{TEXT("dab5352e-a5a7-4793-a7a3-1d4acad6aff2")};

public:
	[[nodiscard]] inline explicit FTargetReceiptBuildWorkerBuilder(FStringView InTargetReceiptPath, const FTargetReceiptBuildWorkerParams& InParams)
		: TargetReceiptPath(InTargetReceiptPath)
		, Params(InParams)
		, RelativeProjectDir([this]
			{
				FString Path = FPaths::ConvertRelativePathToFull(ProjectDir);
				FString RootDir = FPaths::ConvertRelativePathToFull(FPaths::RootDir());
				if (!FPaths::MakePathRelativeTo(Path, *RootDir))
				{
					Path = TEXTVIEW("Project");
				}
				return Path;
			}())
	{
	}

	[[nodiscard]] TOptional<UE::DerivedData::FBuildWorker> Build(UE::DerivedData::IBuildWorkerRegistry& Registry);

private:
	struct FFileMetaData
	{
		UE::FSharedString LocalPath;
		FString RemotePath;
		FDateTime ModTime;
		FSharedBuffer RawData;
		FCompressedBuffer Data;
		bool bExecutable = false;
	};

	[[nodiscard]] bool TryFindTargetReceipt(FStringBuilderBase& OutActiveTargetReceiptPath, FStringBuilderBase& OutActiveWorkerPackagePath);

	[[nodiscard]] FFileMetaData* TryAddFile(FStringView Path);

	[[nodiscard]] bool TryLoadWorkerVersions(FStringView ActiveTargetReceiptPath, FStringView ActiveWorkerPackagePath);

private:
	const FStringView TargetReceiptPath;
	const FTargetReceiptBuildWorkerParams Params;

	FString Name;
	FString HostPlatform;
	FGuid BuildSystemVersion;
	TArray<TTuple<FString, FGuid>> Functions;
	TMap<UE::FSharedString, FFileMetaData> LocalPathToMetaData;
	TMap<FString, FString> Environment;

	UE::FSharedString LocalLaunchPath;
	FDateTime MostRecentFileModTime;

	IFileManager& FileManager = IFileManager::Get();
	const FString EngineDir = FPaths::EngineDir();
	const FString ProjectDir = FPaths::ProjectDir();
	const FString RelativeProjectDir;
	const FStringView EngineDirPrefix = TEXTVIEW("$(EngineDir)");
	const FStringView ProjectDirPrefix = TEXTVIEW("$(ProjectDir)");
};

TOptional<UE::DerivedData::FBuildWorker> FTargetReceiptBuildWorkerBuilder::Build(UE::DerivedData::IBuildWorkerRegistry& Registry)
{
	const double StartTime = FPlatformTime::Seconds();

	TStringBuilder<256> ActiveTargetReceiptPath;
	TStringBuilder<256> ActiveWorkerPackagePath(InPlace, Params.WorkerPackagePath);
	if (!TryFindTargetReceipt(ActiveTargetReceiptPath, ActiveWorkerPackagePath))
	{
		return {};
	}

	const FDateTime TargetReceiptModTime = FileManager.GetTimeStamp(*WriteToString<256>(ActiveTargetReceiptPath));
	const FDateTime WorkerPackageModTime = FileManager.GetTimeStamp(*WriteToString<256>(ActiveWorkerPackagePath));

	if (TargetReceiptModTime == FDateTime::MinValue())
	{
		if (TOptional<UE::DerivedData::FBuildWorker> Worker = Registry.LoadWorker(ActiveWorkerPackagePath))
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Verbose,
				"Target receipt does not exist at '{ActiveTargetReceipt}'. "
				"Using existing build worker package at '{Path}' for '{TargetReceipt}'.",
				ActiveTargetReceiptPath, ActiveWorkerPackagePath, TargetReceiptPath);
			return Worker;
		}
		return {};
	}

	UE_LOGFMT(LogTargetReceiptBuildWorker, Verbose, "Using target receipt '{TargetReceipt}'", ActiveTargetReceiptPath);

	if (WorkerPackageModTime > TargetReceiptModTime && !Params.bAlwaysBuild)
	{
		if (TOptional<UE::DerivedData::FBuildWorker> Worker = Registry.LoadWorker(ActiveWorkerPackagePath))
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Verbose,
				"Target receipt has not been modified since the build worker package was created. "
				"Using existing build worker package at '{Path}' for '{TargetReceipt}'.",
				ActiveWorkerPackagePath, TargetReceiptPath);
			return Worker;
		}
	}

	FTargetReceipt TargetReceipt;
	if (!TargetReceipt.Read(FString(ActiveTargetReceiptPath), /*bExpandVariables*/ false))
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Warning, "Failed to parse target receipt '{TargetReceipt}'", TargetReceiptPath);
		return {};
	}

	if (Params.ShouldAbort && Params.ShouldAbort())
	{
		return {};
	}

	// Store the name and platform early to allow their use during loading and logging.
	Name = TargetReceipt.TargetName;
	HostPlatform = TargetReceipt.Platform;

	if (FFileMetaData* File = TryAddFile(TargetReceipt.Launch))
	{
		File->bExecutable = true;
		LocalLaunchPath = File->LocalPath;
	}
	else
	{
		return {};
	}

	for (const FBuildProduct& BuildProduct : TargetReceipt.BuildProducts)
	{
		if (!Params.bIncludeSymbols && BuildProduct.Type == TEXTVIEW("SymbolFile"))
		{
			continue;
		}
		if (FFileMetaData* File = TryAddFile(BuildProduct.Path))
		{
			File->bExecutable = BuildProduct.Type == TEXTVIEW("Executable");
		}
		else
		{
			return {};
		}
	}

	for (const FRuntimeDependency& RuntimeDependency : TargetReceipt.RuntimeDependencies)
	{
		if (RuntimeDependency.Path.EndsWith(TEXT(".uproject")))
		{
			// uproject files are not needed by build workers.
			continue;
		}
		if (!TryAddFile(RuntimeDependency.Path))
		{
			return {};
		}
	}

	// Scan the additional properties for build worker versions.
	FGuid BuildWorkerReceiptVersion;

	for (const FReceiptProperty& Property : TargetReceipt.AdditionalProperties)
	{
		if (Property.Name == TEXTVIEW("DerivedDataBuildWorkerReceiptVersion"))
		{
			if (!FGuid::Parse(Property.Value, BuildWorkerReceiptVersion))
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
					"Invalid build worker receipt version '{Version}' in target receipt for '{Name}' at path '{TargetReceipt}'.",
					Property.Value, Name, TargetReceiptPath);
				return {};
			}
		}
		else if (Property.Name == TEXTVIEW("DerivedDataBuildWorkerBuildSystemVersion"))
		{
			if (!FGuid::Parse(Property.Value, BuildSystemVersion))
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
					"Invalid build worker build system version '{Version}' in target receipt for '{Name}' at path '{TargetReceipt}'.",
					Property.Value, Name, TargetReceiptPath);
				return {};
			}
		}
		else if (Property.Name == TEXTVIEW("DerivedDataBuildWorkerFunctionVersion"))
		{
			TArray<FStringView, TInlineAllocator<2>> FunctionComponents;
			UE::String::ParseTokens(Property.Value, TEXT(':'), FunctionComponents);
			if (FunctionComponents.Num() != 2)
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
					"Invalid build worker function name and version '{Version}' in target receipt for '{Name}' at path '{TargetReceipt}'.",
					Property.Value, Name, TargetReceiptPath);
				return {};
			}

			FGuid FunctionVersion;
			if (!FGuid::Parse(FunctionComponents[1], FunctionVersion))
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
					"Invalid build worker function version '{Version}' in target receipt for '{Name}' at path '{TargetReceipt}'.",
					FunctionComponents[1], Name, TargetReceiptPath);
				return {};
			}

			Functions.Emplace(FunctionComponents[0], MoveTemp(FunctionVersion));
		}
		else if (Property.Name == TEXTVIEW("UnstagedRuntimeDependency"))
		{
			if (!TryAddFile(Property.Value))
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
					"Unusable unstaged dependency '{Path}' referenced by '{Name}' at '{TargetReceipt}'.", Property.Value, Name, TargetReceiptPath);
				return {};
			}
		}
	}

	if (BuildWorkerReceiptVersion != CurrentWorkerReceiptVersion)
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Verbose,
			"Ignoring target with no build worker receipt version for '{Name}' at path '{TargetReceipt}'...", Name, TargetReceiptPath);
		return {};
	}

	if (Params.ShouldAbort && Params.ShouldAbort())
	{
		return {};
	}

	// Find the most recently modified file referenced by the target receipt.
	MostRecentFileModTime = Algo::MaxElementBy(LocalPathToMetaData,
		Projection(&TPair<UE::FSharedString, FFileMetaData>::Value, &FFileMetaData::ModTime))->Value.ModTime;

	if (WorkerPackageModTime > MostRecentFileModTime && !Params.bAlwaysBuild)
	{
		if (TOptional<UE::DerivedData::FBuildWorker> Worker = Registry.LoadWorker(ActiveWorkerPackagePath))
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Verbose,
				"Files referenced by the target receipt have not been modified since the build worker package was created. "
				"Using existing build worker package at '{Path}' for '{Name}' at path '{TargetReceipt}'.",
				ActiveWorkerPackagePath, Name, TargetReceiptPath);
			return Worker;
		}
	}

	if (!BuildSystemVersion.IsValid() && !TryLoadWorkerVersions(ActiveTargetReceiptPath, ActiveWorkerPackagePath))
	{
		return {};
	}

	if (Params.ShouldAbort && Params.ShouldAbort())
	{
		return {};
	}

	UE_LOGFMT(LogTargetReceiptBuildWorker, Log, "Building a build worker package from {FileCount} files referenced by '{TargetReceipt}'...",
		LocalPathToMetaData.Num(), TargetReceiptPath);

	// Load the referenced files from disk.
	for (TPair<UE::FSharedString, FFileMetaData>& It : LocalPathToMetaData)
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Verbose, "Loading file '{Path}' for '{Name}' at path '{TargetReceipt}'...",
			It.Value.LocalPath, Name, TargetReceiptPath);
		if (TUniquePtr<FArchive> Ar{FileManager.CreateFileReader(*It.Value.LocalPath, FILEREAD_Silent)})
		{
			const int64 TotalSize = FPlatformMath::Max(0, Ar->TotalSize());
			FUniqueBuffer Data = FUniqueBuffer::Alloc(uint64(TotalSize));
			Ar->Serialize(Data.GetData(), TotalSize);
			if (!Ar->Close())
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning, "Failed to read file '{Path}' for '{Name}' at path '{TargetReceipt}'.",
					It.Value.LocalPath, Name, TargetReceiptPath);
				return {};
			}
			It.Value.RawData = Data.MoveToShared();
		}
		else
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Warning, "Failed to open file '{Path}' for '{Name}' at path '{TargetReceipt}'.",
				It.Value.LocalPath, Name, TargetReceiptPath);
			return {};
		}

		if (Params.ShouldAbort && Params.ShouldAbort())
		{
			return {};
		}
	}
	const uint64 TotalFileSize = Algo::TransformAccumulate(LocalPathToMetaData,
		Projection(&TPair<UE::FSharedString, FFileMetaData>::Value, &FFileMetaData::RawData, &FSharedBuffer::GetSize), uint64(0));

	// Hash and optionally compress the files.
	ParallelFor(TEXT("CompressBuildWorker"), LocalPathToMetaData.Num(), /*MinBatchSize*/ 1, [this](int32 Index)
	{
		FFileMetaData& MetaData = LocalPathToMetaData.Get(FSetElementId::FromInteger(Index)).Value;
		if (Params.bSkipCompression)
		{
			MetaData.Data = FCompressedBuffer::Compress(MetaData.RawData, ECompressedBufferCompressor::NotSet, ECompressedBufferCompressionLevel::None);
		}
		else
		{
			MetaData.Data = FCompressedBuffer::Compress(MetaData.RawData, ECompressedBufferCompressor::Kraken, ECompressedBufferCompressionLevel::Normal);
		}
		MetaData.RawData.Reset();
	});

	// Build the build worker package.
	UE::DerivedData::FBuildWorkerBuilder Builder = Registry.CreateWorker();
	Builder.SetName(WriteToUtf8String<64>(Name));
	Builder.SetPath(WriteToUtf8String<256>(LocalPathToMetaData[LocalLaunchPath].RemotePath));
	Builder.SetHostPlatform(WriteToUtf8String<64>(HostPlatform));
	Builder.SetBuildSystemVersion(BuildSystemVersion);
	for (const TTuple<FString, FGuid>& Function : Functions)
	{
		Builder.AddFunction(WriteToUtf8String<64>(Function.Key), Function.Value);
	}
	for (const TPair<UE::FSharedString, FFileMetaData>& It : LocalPathToMetaData)
	{
		if (It.Value.bExecutable)
		{
			Builder.AddExecutable(WriteToUtf8String<256>(It.Value.RemotePath), It.Value.Data);
		}
		else
		{
			Builder.AddFile(WriteToUtf8String<256>(It.Value.RemotePath), It.Value.Data);
		}
	}
	for (const TPair<FString, FString>& Env : Environment)
	{
		Builder.SetEnvironment(WriteToUtf8String<64>(Env.Key), WriteToUtf8String<192>(Env.Value));
	}

	// Build workers require this directory to exist for correct operation.
	Builder.AddDirectory(WriteToUtf8String<64>("Engine/Binaries/", HostPlatform));

	UE::DerivedData::FBuildWorker Worker = Builder.Build(ActiveWorkerPackagePath);

	const double EndTime = FPlatformTime::Seconds();

	const int64 PackageSize = FileManager.FileSize(*ActiveWorkerPackagePath);
	UE_LOGFMT(LogTargetReceiptBuildWorker, Log, "Created build worker package from '{TargetReceipt}' in {Time} seconds. "
		"Package is {PackageSizeMB} MiB. Input is {FileSizeMB} MiB in {FileCount} files.",
		TargetReceiptPath, EndTime - StartTime, PackageSize / 1024 / 1024, TotalFileSize / 1024 / 1024, LocalPathToMetaData.Num());

	return Worker;
}

bool FTargetReceiptBuildWorkerBuilder::TryFindTargetReceipt(
	FStringBuilderBase& OutActiveTargetReceiptPath,
	FStringBuilderBase& OutActiveWorkerPackagePath)
{
	if (TargetReceiptPath.StartsWith(EngineDirPrefix))
	{
		// A target that is expected to be in the engine directory will be found in the project
		// directory if a project was provided at build time. Check both targets and take the one
		// with the more recent modification time.
		const FStringView TargetReceiptRelativePath = TargetReceiptPath.RightChop(EngineDirPrefix.Len() + 1);
		const FString EngineReceiptPath = FPaths::ConvertRelativePathToFull(EngineDir + TargetReceiptRelativePath);
		const FString ProjectReceiptPath = FPaths::ConvertRelativePathToFull(ProjectDir + TargetReceiptRelativePath);
		FDateTime EngineReceiptModTime;
		FDateTime ProjectReceiptModTime;
		FileManager.GetTimeStampPair(*EngineReceiptPath, *ProjectReceiptPath, EngineReceiptModTime, ProjectReceiptModTime);

		FString ActiveSavedDir;
		if (ProjectReceiptModTime > EngineReceiptModTime)
		{
			OutActiveTargetReceiptPath = ProjectReceiptPath;
			ActiveSavedDir = FPaths::ProjectSavedDir();
		}
		else
		{
			OutActiveTargetReceiptPath = EngineReceiptPath;
			ActiveSavedDir = FPaths::EngineSavedDir();
		}

		if (OutActiveWorkerPackagePath.Len() == 0)
		{
			const FString WorkerPackageRelativePath =
				FPaths::Combine(TEXTVIEW("BuildWorkers"), FPathViews::GetBaseFilenameWithPath(TargetReceiptRelativePath)) + TEXTVIEW(".worker");
			if (EngineReceiptModTime == FDateTime::MinValue() && ProjectReceiptModTime == FDateTime::MinValue())
			{
				// When neither target receipt exists, check which build worker package is newer.
				const FString EngineWorkerPackagePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::EngineSavedDir(), WorkerPackageRelativePath));
				const FString ProjectWorkerPackagePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectSavedDir(), WorkerPackageRelativePath));
				FDateTime EngineWorkerPackageModTime;
				FDateTime ProjectWorkerPackageModTime;
				FileManager.GetTimeStampPair(*EngineWorkerPackagePath, *ProjectWorkerPackagePath, EngineWorkerPackageModTime, ProjectWorkerPackageModTime);
				if (ProjectWorkerPackageModTime > EngineWorkerPackageModTime)
				{
					OutActiveTargetReceiptPath = ProjectReceiptPath;
					OutActiveWorkerPackagePath = ProjectWorkerPackagePath;
				}
				else
				{
					OutActiveTargetReceiptPath = EngineReceiptPath;
					OutActiveWorkerPackagePath = EngineWorkerPackagePath;
				}
			}
			else
			{
				OutActiveWorkerPackagePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(ActiveSavedDir, WorkerPackageRelativePath));
			}
		}
		return true;
	}

	if (TargetReceiptPath.StartsWith(ProjectDirPrefix))
	{
		const FStringView TargetReceiptRelativePath = TargetReceiptPath.RightChop(ProjectDirPrefix.Len() + 1);
		OutActiveTargetReceiptPath = FPaths::ConvertRelativePathToFull(ProjectDir + TargetReceiptRelativePath);
		if (OutActiveWorkerPackagePath.Len() == 0)
		{
			OutActiveWorkerPackagePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectSavedDir(),
				TEXTVIEW("BuildWorkers"), FPathViews::GetBaseFilenameWithPath(TargetReceiptRelativePath)) + TEXTVIEW(".worker"));
		}
		return true;
	}

	if (TargetReceiptPath.StartsWith(TEXTVIEW("$(")))
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Warning, "Unknown prefix for target receipt path '{TargetReceipt}'.", TargetReceiptPath);
		return false;
	}

	if (OutActiveWorkerPackagePath.Len() == 0)
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Error, "The build worker package path must be provided because the "
			"target receipt path '{TargetReceipt}' is not prefixed with '{EngineDirPrefix}' or '{ProjectDirPrefix}'.",
			TargetReceiptPath, EngineDirPrefix, ProjectDirPrefix);
		return false;
	}

	OutActiveTargetReceiptPath = TargetReceiptPath;
	return true;
}

FTargetReceiptBuildWorkerBuilder::FFileMetaData* FTargetReceiptBuildWorkerBuilder::TryAddFile(FStringView Path)
{
	if (Path.IsEmpty())
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
			"Empty path found in target receipt for '{Name}' at path '{TargetReceipt}'.", Name, TargetReceiptPath);
		return {};
	}

	TStringBuilder<256> LocalPath;
	TStringBuilder<256> RemotePath;

	if (Path.StartsWith(EngineDirPrefix))
	{
		FPathViews::Append(LocalPath, EngineDir, Path.RightChop(EngineDirPrefix.Len() + 1));
		FPathViews::Append(RemotePath, ANSITEXTVIEW("Engine"), Path.RightChop(EngineDirPrefix.Len() + 1));
	}
	else if (Path.StartsWith(ProjectDirPrefix))
	{
		FPathViews::Append(LocalPath, ProjectDir, Path.RightChop(ProjectDirPrefix.Len() + 1));
		FPathViews::Append(RemotePath, RelativeProjectDir, Path.RightChop(ProjectDirPrefix.Len() + 1));
	}
	else
	{
		if (!Path.StartsWith(TEXTVIEW("$(")))
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
				"Absolute or unprefixed path '{Path}' found in target receipt for '{Name}' at path '{TargetReceipt}'.",
				Path, Name, TargetReceiptPath);
			return {};
		}

		const int32 EndVariableIndex = Path.Find(TEXTVIEW(")"));
		if (EndVariableIndex == INDEX_NONE)
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
				"Expected ')' following '$(' in path '{Path}' in target receipt for '{Name}' at path '{TargetReceipt}'.",
				Path, Name, TargetReceiptPath);
			return {};
		}

		const FStringView RelativePath = Path.RightChop(EndVariableIndex + 2);
		const FStringView EnvVarName = Path.Mid(2, EndVariableIndex - 2);
		const FString EnvVarValue = FPlatformMisc::GetEnvironmentVariable(*WriteToString<64>(EnvVarName));
		if (EnvVarValue.IsEmpty())
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
				"Unknown environment variable '{EnvVar}' in path '{Path}' in target receipt for '{Name}' at path '{TargetReceipt}'.",
				EnvVarName, Path, Name, TargetReceiptPath);
			return {};
		}

		// Depending on an environment variable to locate the file now creates an implicit
		// dependency on the environment variable when executing the worker later.
		// Files within this local directory are remapped to a directory within the worker root
		// that is named the same as the environment variable. That path needs to be added to the
		// environment of the worker to allow it to resolve these files when it executes.
		FPathViews::Append(RemotePath, "..", "..", "..", EnvVarName);
		Environment.Emplace(EnvVarName, RemotePath);
		FPathViews::Append(RemotePath, RelativePath);
		FPathViews::Append(LocalPath, EnvVarValue, RelativePath);
	}

	FDateTime ModTime = FileManager.GetTimeStamp(*LocalPath);
	if (ModTime == FDateTime::MinValue())
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
			"Missing file '{LocalPath}' referenced by target receipt for '{Name}' at path '{TargetReceipt}'.", LocalPath, Name, TargetReceiptPath);
		return {};
	}

	FFileMetaData MetaData;
	MetaData.LocalPath = LocalPath;
	MetaData.RemotePath = RemotePath;
	MetaData.ModTime = ModTime;
	return &LocalPathToMetaData.FindOrAdd(MetaData.LocalPath, MetaData);
}

bool FTargetReceiptBuildWorkerBuilder::TryLoadWorkerVersions(FStringView ActiveTargetReceiptPath, FStringView ActiveWorkerPackagePath)
{
	FCbObject VersionObject;

	TStringBuilder<256> VersionPath(InPlace, ActiveTargetReceiptPath, TEXTVIEW(".BuildVersion"));
	if (!FileManager.FileExists(*VersionPath))
	{
		if (FPlatformProcess::GetBinariesSubdirectory() != HostPlatform)
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Log,
				"The build worker is for platform '{HostPlatform}' and does not have published versions for '{Name}' at path '{TargetReceipt}'.",
				HostPlatform, Name, TargetReceiptPath);
			return false;
		}

		VersionPath = ActiveWorkerPackagePath;
		VersionPath.Append(TEXTVIEW(".BuildVersion"));

		FDateTime VersionFileModificationTime = FileManager.GetTimeStamp(*VersionPath);
		if (VersionFileModificationTime < MostRecentFileModTime)
		{
			UE_LOGFMT(LogTargetReceiptBuildWorker, Verbose,
				"Launching '{LaunchPath}' to report versions for '{Name}'...", LocalLaunchPath, Name);

			TStringBuilder<256> CommandLineParams(InPlace, TEXTVIEW("-Version=\""), VersionPath, TEXTVIEW("\""));
			FProcHandle WorkerProcHandle = FPlatformProcess::CreateProc(
				*FPaths::ConvertRelativePathToFull(FString(LocalLaunchPath)),
				*CommandLineParams,
				/*bLaunchDetached*/ true,
				/*bLaunchHidden*/ true,
				/*bLaunchReallyHidden*/ true,
				/*OutProcessID*/ nullptr,
				/*PriorityModifier*/ 0,
				/*OptionalWorkingDirectory*/ nullptr,
				/*PipeWriteChild*/ nullptr);
			if (!WorkerProcHandle.IsValid())
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
					"Failed to launch build worker '{LaunchPath}' to report versions for '{Name}'.", LocalLaunchPath, Name);
				return false;
			}

			uint64 WorkerStartTime = FPlatformTime::Cycles64();
			while (FPlatformProcess::IsProcRunning(WorkerProcHandle))
			{
				FPlatformProcess::Sleep(0.1f);
				if ((Params.ShouldAbort && Params.ShouldAbort()) || (FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - WorkerStartTime) > 10))
				{
					UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
						"Aborted build worker '{LaunchPath}' version reporting for '{Name}'.", LocalLaunchPath, Name);
					FPlatformProcess::TerminateProc(WorkerProcHandle, /*bKillTree*/ true);
					FPlatformProcess::CloseProc(WorkerProcHandle);
					return false;
				}
			}

			FPlatformProcess::CloseProc(WorkerProcHandle);

			if (FileManager.GetTimeStamp(*VersionPath) < MostRecentFileModTime)
			{
				UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
					"The build worker '{LaunchPath}' did not produce a version file for '{Name}'.", LocalLaunchPath, Name);
				return false;
			}
		}
	}

	if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*VersionPath, FILEREAD_Silent)})
	{
		*Ar << VersionObject;
	}
	else
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
			"Failed to load build worker version file '{VersionPath}' for '{Name}'.", VersionPath, Name);
		return false;
	}

	BuildSystemVersion = VersionObject["BuildSystemVersion"].AsUuid();
	if (!BuildSystemVersion.IsValid())
	{
		UE_LOGFMT(LogTargetReceiptBuildWorker, Warning,
			"Invalid build worker build system version in published version file '{VersionPath}' for '{Name}'.", VersionPath, Name);
		return false;
	}

	for (FCbFieldView FunctionField : VersionObject["Functions"])
	{
		Functions.Emplace(FunctionField["Name"].AsString(), FunctionField["Version"].AsUuid());
	}

	return true;
}

TOptional<UE::DerivedData::FBuildWorker> CreateBuildWorkerFromTargetReceipt(
	FStringView TargetReceiptPath,
	UE::DerivedData::IBuildWorkerRegistry& Registry,
	const FTargetReceiptBuildWorkerParams& Params)
{
	FTargetReceiptBuildWorkerBuilder Builder(TargetReceiptPath, Params);
	return Builder.Build(Registry);
}

#if WITH_EDITOR
class FTargetReceiptBuildWorkerFactory final : public UE::DerivedData::IBuildWorkerFactory
{
public:
	explicit FTargetReceiptBuildWorkerFactory(FStringView TargetReceiptFilePath)
		: TargetReceiptPath(TargetReceiptFilePath)
	{
		IModularFeatures::Get().RegisterModularFeature(UE::DerivedData::IBuildWorkerFactory::FeatureName, this);
	}

	~FTargetReceiptBuildWorkerFactory() final
	{
		IModularFeatures::Get().UnregisterModularFeature(UE::DerivedData::IBuildWorkerFactory::FeatureName, this);
	}

	void CreateWorkers(UE::DerivedData::IBuildWorkerRegistry& Registry) final
	{
		CreateWorkerTask = UE::Tasks::Launch(TEXT("TargetReceiptBuildWorkerFactory"), [this, &Registry]
		{
			const auto ShouldAbort = [this] { return bAborted.load(std::memory_order_relaxed); };
			if (TOptional<UE::DerivedData::FBuildWorker> Worker = CreateBuildWorkerFromTargetReceipt(TargetReceiptPath, Registry, {.ShouldAbort = ShouldAbort}))
			{
				Registry.RegisterWorker(MoveTemp(*Worker), this);
			}
		});
	}

	void AbortCreateWorkers() final
	{
		bAborted.store(true, std::memory_order_relaxed);
		CreateWorkerTask.Wait();
	}

private:
	UE::FSharedString TargetReceiptPath;
	UE::Tasks::FTask CreateWorkerTask;
	std::atomic<bool> bAborted = false;
};
#endif // WITH_EDITOR

FTargetReceiptBuildWorker::FTargetReceiptBuildWorker(FStringView TargetReceiptFilePath)
{
#if WITH_EDITOR
	Factory = MakePimpl<FTargetReceiptBuildWorkerFactory>(TargetReceiptFilePath);
#endif
}

FTargetReceiptBuildWorker::~FTargetReceiptBuildWorker() = default;
