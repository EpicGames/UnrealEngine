// Copyright Epic Games, Inc. All Rights Reserved.

#include "DesktopPlatformWindows.h"
#include "DesktopPlatformPrivate.h"
#include "Misc/FeedbackContextMarkup.h"
#include "WindowsNativeFeedbackContext.h"
#include "Microsoft/COMPointer.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "HAL/FileManager.h"

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <commdlg.h>
	#include <shellapi.h>
	#include <shlobj.h>
	#include <Winver.h>
	#include <LM.h>
	#include <tlhelp32.h>
	#include <Psapi.h>
#include "Windows/HideWindowsPlatformTypes.h"

#pragma comment( lib, "version.lib" )

#define LOCTEXT_NAMESPACE "DesktopPlatform"
#define MAX_FILETYPES_STR 4096
#define MAX_FILENAME_STR 65536 // This buffer has to be big enough to contain the names of all the selected files as well as the null characters between them and the null character at the end

static const TCHAR *InstallationsSubKey = TEXT("SOFTWARE\\Epic Games\\Unreal Engine\\Builds");

bool FDesktopPlatformWindows::OpenFileDialog( const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex )
{
	return FileDialogShared(false, ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, OutFilterIndex );
}

bool FDesktopPlatformWindows::OpenFileDialog( const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	int32 DummyFilterIndex;
	return FileDialogShared(false, ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyFilterIndex );
}

bool FDesktopPlatformWindows::SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames)
{
	int32 DummyFilterIndex = 0;
	return FileDialogShared(true, ParentWindowHandle, DialogTitle, DefaultPath, DefaultFile, FileTypes, Flags, OutFilenames, DummyFilterIndex );
}

bool FDesktopPlatformWindows::OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, FString& OutFolderName)
{
	FScopedSystemModalMode SystemModalScope;

	bool bSuccess = false;

	TComPtr<IFileOpenDialog> FileDialog;
	if (SUCCEEDED(::CoCreateInstance(CLSID_FileOpenDialog, nullptr, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&FileDialog))))
	{
		// Set this up as a folder picker
		{
			DWORD dwFlags = 0;
			FileDialog->GetOptions(&dwFlags);
			FileDialog->SetOptions(dwFlags | FOS_PICKFOLDERS);
		}

		// Set up common settings
		FileDialog->SetTitle(*DialogTitle);
		if (!DefaultPath.IsEmpty())
		{
			// SHCreateItemFromParsingName requires the given path be absolute and use \ rather than / as our normalized paths do
			FString DefaultWindowsPath = FPaths::ConvertRelativePathToFull(DefaultPath);
			DefaultWindowsPath.ReplaceInline(TEXT("/"), TEXT("\\"), ESearchCase::CaseSensitive);

			TComPtr<IShellItem> DefaultPathItem;
			if (SUCCEEDED(::SHCreateItemFromParsingName(*DefaultWindowsPath, nullptr, IID_PPV_ARGS(&DefaultPathItem))))
			{
				FileDialog->SetFolder(DefaultPathItem);
			}
		}

		// Show the picker
		if (SUCCEEDED(FileDialog->Show((HWND)ParentWindowHandle)))
		{
			TComPtr<IShellItem> Result;
			if (SUCCEEDED(FileDialog->GetResult(&Result)))
			{
				PWSTR pFilePath = nullptr;
				if (SUCCEEDED(Result->GetDisplayName(SIGDN_FILESYSPATH, &pFilePath)))
				{
					bSuccess = true;

					OutFolderName = pFilePath;
					FPaths::NormalizeDirectoryName(OutFolderName);

					::CoTaskMemFree(pFilePath);
				}
			}
		}
	}
	
	return bSuccess;
}

bool FDesktopPlatformWindows::OpenFontDialog(const void* ParentWindowHandle, FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags)
{	
	FScopedSystemModalMode SystemModalScope;

	CHOOSEFONT cf;
	ZeroMemory(&cf, sizeof(CHOOSEFONT));

	LOGFONT lf;
	ZeroMemory(&lf, sizeof(LOGFONT));

	cf.lStructSize = sizeof(CHOOSEFONT);
	cf.hwndOwner = (HWND)ParentWindowHandle;
	cf.lpLogFont = &lf;
	cf.Flags = CF_EFFECTS | CF_SCREENFONTS;
	bool bSuccess = !!::ChooseFont(&cf);
	if ( bSuccess )
	{
		HDC DC = ::GetDC( cf.hwndOwner ); 
		const float LogicalPixelsY = static_cast<float>(GetDeviceCaps(DC, LOGPIXELSY));
		const int32 PixelHeight = static_cast<int32>((float) - lf.lfHeight * (72.0f / LogicalPixelsY));	// Always target 72 DPI
		auto FontFlags = EFontImportFlags::None;
		if ( lf.lfUnderline )
		{
			FontFlags |= EFontImportFlags::EnableUnderline;
		}
		if ( lf.lfItalic )
		{
			FontFlags |= EFontImportFlags::EnableItalic;
		}
		if ( lf.lfWeight == FW_BOLD )
		{
			FontFlags |= EFontImportFlags::EnableBold;
		}

		OutFontName = (const TCHAR*)lf.lfFaceName;
		OutHeight = (float)PixelHeight;
		OutFlags = FontFlags;

		::ReleaseDC( cf.hwndOwner, DC ); 
	}
	else
	{
		UE_LOGF(LogDesktopPlatform, Warning, "Error reading results of font dialog.");
	}

	return bSuccess;
}

CA_SUPPRESS(6262)

bool FDesktopPlatformWindows::FileDialogShared(bool bSave, const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex)
{
	FScopedSystemModalMode SystemModalScope;

	bool bSuccess = false;

	TComPtr<IFileDialog> FileDialog;
	if (SUCCEEDED(::CoCreateInstance(bSave ? CLSID_FileSaveDialog : CLSID_FileOpenDialog, nullptr, CLSCTX_INPROC_SERVER, bSave ? IID_IFileSaveDialog : IID_IFileOpenDialog, IID_PPV_ARGS_Helper(&FileDialog))))
	{
		if (bSave)
		{
			// Set the default "filename"
			if (!DefaultFile.IsEmpty())
			{
				FileDialog->SetFileName(*FPaths::GetCleanFilename(DefaultFile));
			}
		}
		else
		{
			// Set this up as a multi-select picker
			if (Flags & EFileDialogFlags::Multiple)
			{
				DWORD dwFlags = 0;
				FileDialog->GetOptions(&dwFlags);
				FileDialog->SetOptions(dwFlags | FOS_ALLOWMULTISELECT);
			}
		}

		// Set up common settings
		FileDialog->SetTitle(*DialogTitle);
		if (!DefaultPath.IsEmpty())
		{
			// SHCreateItemFromParsingName requires the given path be absolute and use \ rather than / as our normalized paths do
			FString DefaultWindowsPath = FPaths::ConvertRelativePathToFull(DefaultPath);
			DefaultWindowsPath.ReplaceInline(TEXT("/"), TEXT("\\"), ESearchCase::CaseSensitive);

			TComPtr<IShellItem> DefaultPathItem;
			if (SUCCEEDED(::SHCreateItemFromParsingName(*DefaultWindowsPath, nullptr, IID_PPV_ARGS(&DefaultPathItem))))
			{
				FileDialog->SetFolder(DefaultPathItem);
			}
		}

		// Set-up the file type filters
		TArray<FString> UnformattedExtensions;
		TArray<COMDLG_FILTERSPEC> FileDialogFilters;
		{
			// Split the given filter string (formatted as "Pair1String1|Pair1String2|Pair2String1|Pair2String2") into the Windows specific filter struct
			FileTypes.ParseIntoArray(UnformattedExtensions, TEXT("|"), true);

			if (UnformattedExtensions.Num() % 2 == 0)
			{
				FileDialogFilters.Reserve(UnformattedExtensions.Num() / 2);
				for (int32 ExtensionIndex = 0; ExtensionIndex < UnformattedExtensions.Num();)
				{
					COMDLG_FILTERSPEC& NewFilterSpec = FileDialogFilters[FileDialogFilters.AddDefaulted()];
					NewFilterSpec.pszName = *UnformattedExtensions[ExtensionIndex++];
					NewFilterSpec.pszSpec = *UnformattedExtensions[ExtensionIndex++];
				}
			}
		}
		FileDialog->SetFileTypes(FileDialogFilters.Num(), FileDialogFilters.GetData());

		// Show the picker
		if (SUCCEEDED(FileDialog->Show((HWND)ParentWindowHandle)))
		{
			OutFilterIndex = 0;
			if (SUCCEEDED(FileDialog->GetFileTypeIndex((UINT*)&OutFilterIndex)))
			{
				OutFilterIndex -= 1; // GetFileTypeIndex returns a 1-based index
			}

			auto AddOutFilename = [&OutFilenames](const FString& InFilename)
			{
				FString& OutFilename = OutFilenames[OutFilenames.Add(InFilename)];
				OutFilename = IFileManager::Get().ConvertToRelativePath(*OutFilename);
				FPaths::NormalizeFilename(OutFilename);
			};

			if (bSave)
			{
				TComPtr<IShellItem> Result;
				if (SUCCEEDED(FileDialog->GetResult(&Result)))
				{
					PWSTR pFilePath = nullptr;
					if (SUCCEEDED(Result->GetDisplayName(SIGDN_FILESYSPATH, &pFilePath)))
					{
						bSuccess = true;

						// Apply the selected extension if the given filename doesn't already have one
						FString SaveFilePath = pFilePath;
						if (FileDialogFilters.IsValidIndex(OutFilterIndex))
						{
							// May have multiple semi-colon separated extensions in the pattern
							const FString ExtensionPattern = FileDialogFilters[OutFilterIndex].pszSpec;
							TArray<FString> SaveExtensions;
							ExtensionPattern.ParseIntoArray(SaveExtensions, TEXT(";"));

							// Build a "clean" version of the selected extension (without the wildcard)
							FString CleanExtension = SaveExtensions[0];
							if (CleanExtension == TEXT("*.*"))
							{
								CleanExtension.Reset();
							}
							else
							{
								int32 WildCardIndex = INDEX_NONE;
								if (CleanExtension.FindChar(TEXT('*'), WildCardIndex))
								{
									CleanExtension.RightChopInline(WildCardIndex + 1, EAllowShrinking::No);
								}
							}

							// We need to split these before testing the extension to avoid anything within the path being treated as a file extension
							FString SaveFileName = FPaths::GetCleanFilename(SaveFilePath);
							SaveFilePath = FPaths::GetPath(SaveFilePath);

							// Apply the extension if the file name doesn't already have one
							if (FPaths::GetExtension(SaveFileName).IsEmpty() && !CleanExtension.IsEmpty())
							{
								SaveFileName = FPaths::SetExtension(SaveFileName, CleanExtension);
							}

							SaveFilePath /= SaveFileName;
						}
						AddOutFilename(SaveFilePath);

						::CoTaskMemFree(pFilePath);
					}
				}
			}
			else
			{
				IFileOpenDialog* FileOpenDialog = static_cast<IFileOpenDialog*>(FileDialog.Get());

				TComPtr<IShellItemArray> Results;
				if (SUCCEEDED(FileOpenDialog->GetResults(&Results)))
				{
					DWORD NumResults = 0;
					Results->GetCount(&NumResults);
					for (DWORD ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
					{
						TComPtr<IShellItem> Result;
						if (SUCCEEDED(Results->GetItemAt(ResultIndex, &Result)))
						{
							PWSTR pFilePath = nullptr;
							if (SUCCEEDED(Result->GetDisplayName(SIGDN_FILESYSPATH, &pFilePath)))
							{
								bSuccess = true;
								AddOutFilename(pFilePath);
								::CoTaskMemFree(pFilePath);
							}
						}
					}
				}
			}
		}
	}

	return bSuccess;
}

bool FDesktopPlatformWindows::RegisterEngineInstallation(const FString &RootDir, FString &OutIdentifier)
{
	bool bRes = false;
	if(IsValidRootDirectory(RootDir))
	{
		HKEY hRootKey;
		if(RegCreateKeyEx(HKEY_CURRENT_USER, InstallationsSubKey, 0, NULL, REG_OPTION_NON_VOLATILE, KEY_ALL_ACCESS, NULL, &hRootKey, NULL) == ERROR_SUCCESS)
		{
			FString NewIdentifier = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensInBraces);
			LRESULT SetResult = RegSetValueEx(hRootKey, *NewIdentifier, 0, REG_SZ, (const BYTE*)*RootDir, (RootDir.Len() + 1) * sizeof(TCHAR));
			RegCloseKey(hRootKey);

			if(SetResult == ERROR_SUCCESS)
			{
				OutIdentifier = NewIdentifier;
				bRes = true;
			}
		}
	}
	return bRes;
}

void FDesktopPlatformWindows::EnumerateEngineInstallations(TMap<FString, FString> &OutInstallations)
{
	// Enumerate the binary installations
	EnumerateLauncherEngineInstallations(OutInstallations);

	// Enumerate the per-user installations
	HKEY hKey;
	if (RegOpenKeyEx(HKEY_CURRENT_USER, InstallationsSubKey, 0, KEY_ALL_ACCESS, &hKey) == ERROR_SUCCESS)
	{
		// Get a list of all the directories
		TArray<FString> UniqueDirectories;
		OutInstallations.GenerateValueArray(UniqueDirectories);

		// Enumerate all the installations
		TArray<FString> InvalidKeys;
		for (::DWORD Index = 0;; Index++)
		{
			TCHAR ValueName[256];
			TCHAR ValueData[MAX_PATH];
			::DWORD ValueType = 0;
			::DWORD ValueNameLength = sizeof(ValueName) / sizeof(ValueName[0]);
			::DWORD ValueDataSize = sizeof(ValueData);

			LRESULT Result = RegEnumValue(hKey, Index, ValueName, &ValueNameLength, NULL, &ValueType, (BYTE*)&ValueData[0], &ValueDataSize);
			if(Result == ERROR_SUCCESS)
			{
				int32 ValueDataLength = ValueDataSize / sizeof(TCHAR);
				if(ValueDataLength > 0 && ValueData[ValueDataLength - 1] == 0) ValueDataLength--;

				FString NormalizedInstalledDirectory = FString::ConstructFromPtrSize(ValueData, ValueDataLength);
				FPaths::NormalizeDirectoryName(NormalizedInstalledDirectory);
				FPaths::CollapseRelativeDirectories(NormalizedInstalledDirectory);

				if(IsValidRootDirectory(NormalizedInstalledDirectory) && !UniqueDirectories.Contains(NormalizedInstalledDirectory))
				{
					OutInstallations.Add(ValueName, NormalizedInstalledDirectory);
					UniqueDirectories.Add(NormalizedInstalledDirectory);
				}
				else
				{
					InvalidKeys.Add(ValueName);
				}
			}
			else if(Result == ERROR_NO_MORE_ITEMS)
			{
				break;
			}
		}

		// Remove all the keys which weren't valid
		for(const FString &InvalidKey: InvalidKeys)
		{
			RegDeleteValue(hKey, *InvalidKey);
		}

		RegCloseKey(hKey);
	}
}

bool FDesktopPlatformWindows::IsSourceDistribution(const FString &RootDir)
{
	// Check for the existence of a GenerateProjectFiles.bat file. This allows compatibility with the GitHub 4.0 release. Guard it against a check for Build.version to skip it in newer engine versions.
	FString BuildVersionPath = RootDir / TEXT("Engine/Build/Build.version");
	if (!IFileManager::Get().FileExists(*BuildVersionPath))
	{
		FString GenerateProjectFilesPath = RootDir / TEXT("GenerateProjectFiles.bat");
		if (IFileManager::Get().FileSize(*GenerateProjectFilesPath) >= 0)
		{
			return true;
		}
	}

	// Otherwise use the default test
	return FDesktopPlatformBase::IsSourceDistribution(RootDir);
}

bool FDesktopPlatformWindows::VerifyFileAssociations()
{
	TIndirectArray<FRegistryRootedKey> Keys;
	GetRequiredRegistrySettings(Keys);

	for (int32 Idx = 0; Idx < Keys.Num(); Idx++)
	{
		if (!Keys[Idx].IsUpToDate(true))
		{
			return false;
		}
	}

	return true;
}

bool FDesktopPlatformWindows::UpdateFileAssociations()
{
	TIndirectArray<FRegistryRootedKey> Keys;
	GetRequiredRegistrySettings(Keys);

	for (int32 Idx = 0; Idx < Keys.Num(); Idx++)
	{
		if (!Keys[Idx].Write(true))
		{
			return false;
		}
	}

	return true;
}

bool FDesktopPlatformWindows::OpenProject(const FString &ProjectFileName)
{
	// Get the project filename in a native format
	FString PlatformProjectFileName = ProjectFileName;
	FPaths::MakePlatformFilename(PlatformProjectFileName);

	// Always treat projects as an Unreal.ProjectFile, don't allow the user overriding the file association to take effect
	SHELLEXECUTEINFO Info;
	ZeroMemory(&Info, sizeof(Info));
	Info.cbSize = sizeof(Info);
	Info.fMask = SEE_MASK_CLASSNAME;
	Info.lpVerb = TEXT("open");
	Info.nShow = SW_SHOWNORMAL;
	Info.lpFile = *PlatformProjectFileName;
	Info.lpClass = TEXT("Unreal.ProjectFile");
	return ::ShellExecuteExW(&Info) != 0;
}

bool FDesktopPlatformWindows::RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutExitCode)
{
	OutExitCode = 1;

	// Get the path to UBT
	FString UnrealBuildToolPath = GetUnrealBuildToolExecutableFilename(RootDir);
	if(IFileManager::Get().FileSize(*UnrealBuildToolPath) < 0)
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("Couldn't find UnrealBuildTool at '%s'"), *UnrealBuildToolPath);
		return false;
	}

	// Write the output
	Warn->Logf(TEXT("Running %s %s"), *UnrealBuildToolPath, *Arguments);

	// Spawn UBT
	return FFeedbackContextMarkup::PipeProcessOutput(Description, UnrealBuildToolPath, Arguments, Warn, &OutExitCode) && OutExitCode == 0;
}

FFeedbackContext* FDesktopPlatformWindows::GetNativeFeedbackContext()
{
	static FWindowsNativeFeedbackContext FeedbackContext;
	return &FeedbackContext;
}

FString FDesktopPlatformWindows::GetUserTempPath()
{
	return FString(FPlatformProcess::UserTempDir());
}

void FDesktopPlatformWindows::GetRequiredRegistrySettings(TIndirectArray<FRegistryRootedKey> &RootedKeys)
{
	// Get the path to VersionSelector.exe. If we're running from UnrealVersionSelector itself, try to stick with the current configuration.
	FString DefaultVersionSelectorName = FPlatformProcess::ExecutableName(false);
	if (!DefaultVersionSelectorName.StartsWith(TEXT("UnrealVersionSelector")))
	{
		DefaultVersionSelectorName = TEXT("UnrealVersionSelector.exe");
	}
	FString ExecutableFileName = FPaths::ConvertRelativePathToFull(FPaths::EngineDir()) / TEXT("Binaries/Win64") / DefaultVersionSelectorName;

	// Defer to UnrealVersionSelector.exe in a launcher installation if it's got the same version number or greater.
	FString InstallDir;
	if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\EpicGames\\Unreal Engine"), TEXT("INSTALLDIR"), InstallDir) && (InstallDir.Len() > 0))
	{
		FString NormalizedInstallDir = InstallDir;
		FPaths::NormalizeDirectoryName(NormalizedInstallDir);

		FString InstalledExecutableFileName = NormalizedInstallDir / FString("Launcher/Engine/Binaries/Win64/UnrealVersionSelector.exe");
		if(GetShellIntegrationVersion(InstalledExecutableFileName) == GetShellIntegrationVersion(ExecutableFileName))
		{
			ExecutableFileName = InstalledExecutableFileName;
		}
	}

	// Get the path to the executable
	FPaths::MakePlatformFilename(ExecutableFileName);
	FString QuotedExecutableFileName = FString(TEXT("\"")) + ExecutableFileName + FString(TEXT("\""));

	// HKCU\SOFTWARE\Classes\.uproject
	FRegistryRootedKey *UserRootExtensionKey = new FRegistryRootedKey(HKEY_CURRENT_USER, TEXT("SOFTWARE\\Classes\\.uproject"));
	RootedKeys.Add(UserRootExtensionKey);

	// HKLM\SOFTWARE\Classes\.uproject
	FRegistryRootedKey *RootExtensionKey = new FRegistryRootedKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Classes\\.uproject"));
	RootExtensionKey->Key = MakeUnique<FRegistryKey>();
	RootExtensionKey->Key->SetValue(TEXT(""), TEXT("Unreal.ProjectFile"));
	RootedKeys.Add(RootExtensionKey);

	// HKLM\SOFTWARE\Classes\Unreal.ProjectFile
	FRegistryRootedKey *RootFileTypeKey = new FRegistryRootedKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Classes\\Unreal.ProjectFile"));
	RootFileTypeKey->Key = MakeUnique<FRegistryKey>();
	RootFileTypeKey->Key->SetValue(TEXT(""), TEXT("Unreal Engine Project File"));
	RootFileTypeKey->Key->FindOrAddKey(L"DefaultIcon")->SetValue(TEXT(""), QuotedExecutableFileName);
	RootedKeys.Add(RootFileTypeKey);

	// HKLM\SOFTWARE\Classes\Unreal.ProjectFile\shell
	FRegistryKey *ShellKey = RootFileTypeKey->Key->FindOrAddKey(TEXT("shell"));

	// HKLM\SOFTWARE\Classes\Unreal.ProjectFile\shell\open
	FRegistryKey *ShellOpenKey = ShellKey->FindOrAddKey(TEXT("open"));
	ShellOpenKey->SetValue(L"", L"Open");
	ShellOpenKey->FindOrAddKey(L"command")->SetValue(L"", QuotedExecutableFileName + FString(TEXT(" /editor \"%1\"")));

	// HKLM\SOFTWARE\Classes\Unreal.ProjectFile\shell\run
	FRegistryKey *ShellRunKey = ShellKey->FindOrAddKey(TEXT("run"));
	ShellRunKey->SetValue(TEXT(""), TEXT("Launch game"));
	ShellRunKey->SetValue(TEXT("Icon"), QuotedExecutableFileName);
	ShellRunKey->FindOrAddKey(L"command")->SetValue(TEXT(""), QuotedExecutableFileName + FString(TEXT(" /game \"%1\"")));

	// HKLM\SOFTWARE\Classes\Unreal.ProjectFile\shell\rungenproj
	FRegistryKey *ShellProjectKey = ShellKey->FindOrAddKey(TEXT("rungenproj"));
	ShellProjectKey->SetValue(L"", L"Generate Visual Studio project files");
	ShellProjectKey->SetValue(L"Icon", QuotedExecutableFileName);
	ShellProjectKey->FindOrAddKey(L"command")->SetValue(TEXT(""), QuotedExecutableFileName + FString(TEXT(" /projectfiles \"%1\"")));

	// HKLM\SOFTWARE\Classes\Unreal.ProjectFile\shell\switchversion
	FRegistryKey *ShellVersionKey = ShellKey->FindOrAddKey(TEXT("switchversion"));
	ShellVersionKey->SetValue(TEXT(""), TEXT("Switch Unreal Engine version..."));
	ShellVersionKey->SetValue(TEXT("Icon"), QuotedExecutableFileName);
	ShellVersionKey->FindOrAddKey(L"command")->SetValue(TEXT(""), QuotedExecutableFileName + FString(TEXT(" /switchversion \"%1\"")));

	// If the user has manually selected something other than our extension, we need to delete it. Explorer explicitly disables set access on that keys in that folder, but we can delete the whole thing.
	const TCHAR* UserChoicePath = TEXT("Software\\Microsoft\\Windows\\CurrentVersion\\Explorer\\FileExts\\.uproject\\UserChoice");

	// Figure out whether we need to delete it. If it's already set to our own ProgId, leave it alone.
	bool bDeleteUserChoiceKey = true;
	HKEY hUserChoiceKey;
	if(RegOpenKeyEx(HKEY_CURRENT_USER, UserChoicePath, 0, KEY_READ, &hUserChoiceKey) == S_OK)
	{
		TCHAR ProgId[128];
		::DWORD ProgIdSize = sizeof(ProgId);
		::DWORD ProgIdType = 0;
		if(RegQueryValueEx(hUserChoiceKey, TEXT("Progid"), NULL, &ProgIdType, (BYTE*)ProgId, &ProgIdSize) == ERROR_SUCCESS && ProgIdType == REG_SZ)
		{
			bDeleteUserChoiceKey = (FCString::Strcmp(ProgId, TEXT("Unreal.ProjectFile")) != 0);
		}
		RegCloseKey(hUserChoiceKey);
	}
	if(bDeleteUserChoiceKey)
	{
		RootedKeys.Add(new FRegistryRootedKey(HKEY_CURRENT_USER, UserChoicePath));
	}
}

int32 FDesktopPlatformWindows::GetShellIntegrationVersion(const FString &FileName)
{
	::DWORD VersionInfoSize = GetFileVersionInfoSize(*FileName, NULL);
	if (VersionInfoSize != 0)
	{
		TArray<uint8> VersionInfo;
		VersionInfo.AddUninitialized(VersionInfoSize);
		if (GetFileVersionInfo(*FileName, NULL, VersionInfoSize, VersionInfo.GetData()))
		{
			TCHAR *ShellVersion;
			::UINT ShellVersionLen;
			if (VerQueryValue(VersionInfo.GetData(), TEXT("\\StringFileInfo\\040904b0\\ShellIntegrationVersion"), (LPVOID*)&ShellVersion, &ShellVersionLen))
			{
				TCHAR *ShellVersionEnd;
				int32 Version = FCString::Strtoi(ShellVersion, &ShellVersionEnd, 10);
				if(*ShellVersionEnd == 0)
				{
					return Version;
				}
			}
		}
	}
	return 0;
}

FString FDesktopPlatformWindows::GetOidcTokenExecutableFilename(const FString& RootDir) const
{
	#if !defined(UE_NO_ENGINE_OIDC) || UE_NO_ENGINE_OIDC == 0
	return FPaths::ConvertRelativePathToFull(RootDir / TEXT("Engine/Binaries/DotNET/OidcToken/win-x64/OidcToken.exe"));
	#else
	return TEXT("OidcToken.exe");
	#endif
}

#undef LOCTEXT_NAMESPACE
