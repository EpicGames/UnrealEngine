// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/IndirectArray.h"
#include "CoreMinimal.h"
#include "DesktopPlatformBase.h"
#include "Windows/WindowsHWrapper.h"
#include "WindowsRegistry.h"

class FDesktopPlatformWindows : public FDesktopPlatformBase
{
public:
	// IDesktopPlatform Implementation
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames) override;
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex) override;
	virtual bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames) override;
	virtual bool OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, FString& OutFolderName) override;
	virtual bool OpenFontDialog(const void* ParentWindowHandle, FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags) override;

	virtual bool RegisterEngineInstallation(const FString &RootDir, FString &OutIdentifier) override;
	virtual void EnumerateEngineInstallations(TMap<FString, FString> &OutInstallations) override;

	virtual bool IsSourceDistribution(const FString &RootDir) override;

	virtual bool VerifyFileAssociations() override;
	virtual bool UpdateFileAssociations() override;

	virtual bool OpenProject(const FString &ProjectFileName) override;

	using FDesktopPlatformBase::RunUnrealBuildTool;
	virtual bool RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutExitCode) override;

	virtual FFeedbackContext* GetNativeFeedbackContext() override;

	virtual FString GetUserTempPath() override;

protected:
	virtual FString GetOidcTokenExecutableFilename(const FString& RootDir) const override;

private:
	bool FileDialogShared(bool bSave, const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex);
	void GetRequiredRegistrySettings(TIndirectArray<FRegistryRootedKey> &RootedKeys);
	int32 GetShellIntegrationVersion(const FString &FileName);
};

typedef FDesktopPlatformWindows FDesktopPlatform;
