// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsNativeFeedbackContext.h"
#include "HAL/ThreadHeartBeat.h"
#include "Logging/StructuredLog.h"
#include "Misc/CoreMisc.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/App.h"
#include "Internationalization/Internationalization.h"
#include "Windows/WindowsPlatformApplicationMisc.h"

#include "Windows/AllowWindowsPlatformTypes.h"

FWindowsNativeFeedbackContext::FWindowsNativeFeedbackContext()
	: FFeedbackContext()
	, Context( NULL )
	, hThread( NULL )
	, hCloseEvent( NULL )
	, hUpdateEvent( NULL )
	, Progress( 0.0f )
	, bReceivedUserCancel( false )
	, bShowCancelButton( false )
{
}

FWindowsNativeFeedbackContext::~FWindowsNativeFeedbackContext()
{
	DestroySlowTaskWindow();
}

void FWindowsNativeFeedbackContext::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Serialize(V, Verbosity, Category, -1.0);
}

void FWindowsNativeFeedbackContext::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	FFeedbackContext::Serialize(V, Verbosity, Category, Time);

	// Buffer up the output during a slow task so that we can dump it all to the log console if the show log button is clicked
	if (GIsSlowTask)
	{
		FScopeLock Lock(&CriticalSection);
		if (hThread)
		{
			LogOutput += V;
			LogOutput += TEXT("\r\n");
			SetEvent(hUpdateEvent);
		}
	}
}

void FWindowsNativeFeedbackContext::SerializeRecord(const UE::FLogRecord& Record)
{
	FFeedbackContext::SerializeRecord(Record);

	// Buffer up the output during a slow task so that we can dump it all to the log console if the show log button is clicked
	if (GIsSlowTask)
	{
		TStringBuilder<512> Message;
		Record.FormatMessageTo(Message);
		Message.Append(TEXTVIEW("\r\n"));
		FScopeLock Lock(&CriticalSection);
		if (hThread)
		{
			LogOutput += Message.ToView();
			SetEvent(hUpdateEvent);
		}
	}
}

bool FWindowsNativeFeedbackContext::ReceivedUserCancel()
{
	FScopeLock Lock(&CriticalSection);
	return bReceivedUserCancel;
}

void FWindowsNativeFeedbackContext::StartSlowTask( const FText& Task, bool bShouldShowCancelButton )
{
	FFeedbackContext::StartSlowTask( Task, bShouldShowCancelButton );
	CreateSlowTaskWindow(Task, bShouldShowCancelButton);
}
void FWindowsNativeFeedbackContext::FinalizeSlowTask( )
{
	FFeedbackContext::FinalizeSlowTask( );
	DestroySlowTaskWindow();
}

void FWindowsNativeFeedbackContext::ProgressReported( const float TotalProgressInterp, FText DisplayMessage )
{
	FScopeLock Lock(&CriticalSection);
	if(hThread != NULL)
	{
		Progress = TotalProgressInterp;
		Status = DisplayMessage.ToString();

		SetEvent(hUpdateEvent);
	}
}

FContextSupplier* FWindowsNativeFeedbackContext::GetContext() const
{
	return Context;
}

void FWindowsNativeFeedbackContext::SetContext( FContextSupplier* InSupplier )
{
	Context = InSupplier;
}

void FWindowsNativeFeedbackContext::CreateSlowTaskWindow(const FText &InStatus, bool bInShowCancelButton)
{
	FScopeLock Lock(&CriticalSection);
	if(hThread == NULL && !GIsSilent && !FApp::IsUnattended() && !IsRunningCommandlet())
	{
		Status = InStatus.ToString();
		Progress = 0.0f;
		LogOutput.Empty();

		bReceivedUserCancel = false;
		bShowCancelButton = bInShowCancelButton;

		hCloseEvent = CreateEvent(NULL, TRUE, FALSE, NULL);
		hUpdateEvent = CreateEvent(NULL, FALSE, FALSE, NULL);
		hThread = CreateThread(NULL, 0, &SlowTaskThreadProc, this, 0, NULL);
	}
}

void FWindowsNativeFeedbackContext::DestroySlowTaskWindow()
{
	FScopeLock Lock(&CriticalSection);
	if(hThread != NULL)
	{
		SetEvent(hCloseEvent);

		CriticalSection.Unlock();
		WaitForSingleObject(hThread, INFINITE);
		CriticalSection.Lock();

		CloseHandle(hThread);
		hThread = NULL;

		CloseHandle(hCloseEvent);
		hCloseEvent = NULL;

		CloseHandle(hUpdateEvent);
		hUpdateEvent = NULL;

		LogOutput.Empty();
	}
}

DWORD FWindowsNativeFeedbackContext::SlowTaskThreadProc(void* ThreadParam)
{
	FWindowsNativeFeedbackContext* Context = (FWindowsNativeFeedbackContext*)ThreadParam;

	HINSTANCE HInstance = (HINSTANCE)GetModuleHandle(NULL);

	extern APPLICATIONCORE_API HWND GetSplashScreenWindowHandle();
	HWND ParentWindowHandle = GetSplashScreenWindowHandle();

	WNDCLASSEX WndClassEx;
	ZeroMemory(&WndClassEx, sizeof(WndClassEx));
	WndClassEx.cbSize = sizeof(WndClassEx);
	WndClassEx.style = CS_HREDRAW | CS_VREDRAW | (Context->bShowCancelButton? 0 : CS_NOCLOSE);
	WndClassEx.lpfnWndProc = &SlowTaskWindowProc;
	WndClassEx.hIcon = LoadIcon(HInstance, MAKEINTRESOURCE(FWindowsPlatformApplicationMisc::GetAppIcon()));
	WndClassEx.hCursor = LoadCursor(NULL, IDC_ARROW);
	WndClassEx.hInstance = HInstance;
	WndClassEx.hbrBackground = (HBRUSH)(COLOR_BTNFACE + 1);
	WndClassEx.lpszClassName = TEXT("FFeedbackContextWindows");
	ATOM WndClassAtom = RegisterClassEx(&WndClassEx);

	NONCLIENTMETRICS NonClientMetrics;
	NonClientMetrics.cbSize = sizeof(NonClientMetrics);
	SystemParametersInfo(SPI_GETNONCLIENTMETRICS, sizeof(NonClientMetrics), &NonClientMetrics, 0);
	HANDLE hFont = CreateFontIndirect(&NonClientMetrics.lfMessageFont);

	int FontHeight = -MulDiv(8, GetDeviceCaps(GetDC(NULL), LOGPIXELSY), 72);
	HANDLE hLogFont = CreateFontW(FontHeight, 0, 0, 0, FW_NORMAL, FALSE, FALSE, FALSE, ANSI_CHARSET, OUT_DEFAULT_PRECIS, CLIP_DEFAULT_PRECIS, ANTIALIASED_QUALITY, FIXED_PITCH | FF_MODERN, TEXT("Courier New"));

	TEXTMETRIC TextMetric;
	HDC hDC = CreateCompatibleDC(NULL);
	HGDIOBJ hPrevObj = SelectObject(hDC, hFont);
	GetTextMetrics(hDC, &TextMetric);
	SelectObject(hDC, hPrevObj);
	DeleteDC(hDC);

	FWindowParams Params;
	Params.Context = Context;
	Params.ScaleX = TextMetric.tmAveCharWidth;
	Params.ScaleY = TextMetric.tmHeight;
	Params.StandardW = Params.ScaleX * 80;
	Params.StandardH = Params.ScaleY * 4;
	Params.bLogVisible = false;

	DWORD WindowStyle = WS_VISIBLE | WS_CAPTION | WS_SYSMENU | WS_THICKFRAME;

	RECT WindowRect;
	ZeroMemory(&WindowRect, sizeof(WindowRect));
	WindowRect.left = (GetSystemMetrics(SM_CXSCREEN) - Params.StandardW) / 2;
	WindowRect.top = (GetSystemMetrics(SM_CYSCREEN) - Params.StandardH) / 2;
	WindowRect.right = WindowRect.left + Params.StandardW;
	WindowRect.bottom = WindowRect.top + Params.StandardH;
	AdjustWindowRectEx(&WindowRect, WindowStyle, 0, 0);

	const TCHAR* WindowClassName = MAKEINTATOM( WndClassAtom );
	HWND hWnd = CreateWindow(WindowClassName, TEXT("Unreal Engine"), WindowStyle, WindowRect.left, WindowRect.top, WindowRect.right - WindowRect.left, WindowRect.bottom - WindowRect.top, ParentWindowHandle, NULL, HInstance, NULL);
	SetWindowLongPtr(hWnd, GWLP_USERDATA, (LONG_PTR)&Params);
	SendMessageW(hWnd, WM_SETFONT, (WPARAM)hFont, 0);

	HWND hWndOpenLog = CreateWindow(WC_BUTTON, TEXT("Show log"), BS_CENTER | BS_VCENTER | BS_PUSHBUTTON | BS_TEXT | WS_CHILD | WS_VISIBLE, 10, 10, 10, 10, hWnd, (HMENU)ShowLogCtlId, HInstance, NULL);
	SendMessageW(hWndOpenLog, WM_SETFONT, (WPARAM)hFont, 0);

	HWND hWndStatus = CreateWindow(WC_STATIC, TEXT(""), SS_CENTER | WS_CHILD | WS_VISIBLE, 10, 10, 10, 10, hWnd, (HMENU)StatusCtlId, HInstance, NULL);
	SendMessageW(hWndStatus, WM_SETFONT, (WPARAM)hFont, 0);

	HWND hWndProgress = CreateWindowEx(0, PROGRESS_CLASS, TEXT(""), WS_CHILD | WS_VISIBLE, 10, 10, 10, 10, hWnd, (HMENU)ProgressCtlId, HInstance, NULL);
	SendMessageW(hWndProgress, PBM_SETRANGE32, 0, 1000);

	HWND hWndLogOutput = CreateWindowEx(WS_EX_STATICEDGE, WC_EDIT, TEXT(""), ES_MULTILINE | ES_READONLY | WS_HSCROLL | WS_VSCROLL | WS_CHILD | WS_VISIBLE, 10, 10, 10, 10, hWnd, (HMENU)LogOutputCtlId, HInstance, NULL);
	SendMessageW(hWndLogOutput, WM_SETFONT, (WPARAM)hLogFont, 0);

	LayoutControls(hWnd, &Params);
	SetEvent(Context->hUpdateEvent);

	ShowWindow(hWnd, SW_SHOW);
	UpdateWindow(hWnd);

	SetForegroundWindow(hWnd);

	FString PrevStatus;
	float PrevProgress = 0.0f;
	int32 PrevLogOutputLength = 0;
	for(;;)
	{
		HANDLE Handles[] = { Context->hCloseEvent, Context->hUpdateEvent };
		DWORD Result = MsgWaitForMultipleObjects(2, Handles, 0, INFINITE, QS_ALLEVENTS);
		if(Result == WAIT_OBJECT_0)
		{
			break;
		}
		else if(Result == WAIT_OBJECT_0 + 1)
		{
			FScopeLock Lock(&Context->CriticalSection);
			if(Context->Status != PrevStatus)
			{
				SetWindowText(hWndStatus, *Context->Status);
				PrevStatus = Context->Status;
			}
			if(Context->Progress != PrevProgress)
			{
				SendMessageW(hWndProgress, PBM_SETPOS, (int32)(Context->Progress * 1000.0f), 0);
				PrevProgress = Context->Progress;
			}
			if(Context->LogOutput.Len() > PrevLogOutputLength)
			{
				SendMessageW(hWndLogOutput, EM_SETSEL, PrevLogOutputLength, PrevLogOutputLength);
				SendMessageW(hWndLogOutput, EM_REPLACESEL, FALSE, (LPARAM)(*Context->LogOutput + PrevLogOutputLength));
				SendMessageW(hWndLogOutput, EM_SCROLLCARET, 0, 0);
				PrevLogOutputLength = Context->LogOutput.Len();
			}
		}

		MSG Msg;
		while(PeekMessage(&Msg, NULL, 0, 0, PM_REMOVE))
		{
			TranslateMessage(&Msg);
			DispatchMessage(&Msg);
		}
	}

	DestroyWindow(hWnd);
	DeleteObject(hLogFont);
	DeleteObject(hFont);
	UnregisterClass(WindowClassName, HInstance);

	return 0;
}

LRESULT CALLBACK FWindowsNativeFeedbackContext::SlowTaskWindowProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
{
	switch (Msg)
	{
	case WM_COMMAND:
		if(wParam == ShowLogCtlId)
		{
			FWindowParams *Params = (FWindowParams*)GetWindowLongPtr(hWnd, GWLP_USERDATA);
			Params->bLogVisible ^= true;

			RECT WindowRect;
			GetClientRect(hWnd, &WindowRect);
			WindowRect.bottom = Params->StandardH + (Params->bLogVisible? Params->ScaleY * 10 : 0);
			AdjustWindowRectEx(&WindowRect, GetWindowLong(hWnd, GWL_STYLE), 0, 0);
			SetWindowPos(hWnd, NULL, 0, 0, WindowRect.right - WindowRect.left, WindowRect.bottom - WindowRect.top, SWP_NOZORDER | SWP_NOMOVE);

			SetDlgItemText(hWnd, ShowLogCtlId, Params->bLogVisible? TEXT("Hide log") : TEXT("Show log"));

			ShowWindow(GetDlgItem(hWnd, LogOutputCtlId), Params->bLogVisible? SW_SHOW : SW_HIDE);
			LayoutControls(hWnd, Params);
		}
		return 0;
	case WM_SIZE:
		{
			FWindowParams *Params = (FWindowParams*)GetWindowLongPtr(hWnd, GWLP_USERDATA);
			if(Params != NULL)
			{
				LayoutControls(hWnd, Params);
			}
		}
		return 0;
	case WM_GETMINMAXINFO:
		{
			FWindowParams *Params = (FWindowParams*)GetWindowLongPtr(hWnd, GWLP_USERDATA);
			if(Params != NULL)
			{
				RECT WindowRect;
				SetRect(&WindowRect, 0, 0, Params->StandardW, Params->StandardH + (Params->bLogVisible? (Params->ScaleY * 5) : 0));
				AdjustWindowRectEx(&WindowRect, WS_CAPTION, 0, 0);

				MINMAXINFO *MinMaxInfo = (MINMAXINFO*)lParam;
				MinMaxInfo->ptMinTrackSize.x = WindowRect.right - WindowRect.left;
				MinMaxInfo->ptMinTrackSize.y = WindowRect.bottom - WindowRect.top;

				if(!Params->bLogVisible)
				{
					MinMaxInfo->ptMaxTrackSize.y = MinMaxInfo->ptMinTrackSize.y;
				}
			}
		}
		return 0;
	case WM_CLOSE:
		{
			FWindowParams *Params = (FWindowParams*)GetWindowLongPtr(hWnd, GWLP_USERDATA);
			FScopeLock Lock(&Params->Context->CriticalSection);
			Params->Context->bReceivedUserCancel = true;
		}
		return 0;
	}
	return DefWindowProc(hWnd, Msg, wParam, lParam);
}

void FWindowsNativeFeedbackContext::LayoutControls(HWND hWnd, const FWindowParams* Params)
{
	RECT ClientRect;
	GetClientRect(hWnd, &ClientRect);

	int32 MarginW = Params->ScaleX * 2;
	int32 MarginH = Params->ScaleY;

	int32 SplitX = ClientRect.right - (Params->ScaleX * 15);
	int32 SplitY = Params->ScaleY * 4;

	int32 ButtonH = (Params->ScaleY * 7) / 4;

	HWND hWndOpenLog = GetDlgItem(hWnd, ShowLogCtlId);
	MoveWindow(hWndOpenLog, SplitX, (SplitY - ButtonH) / 2, ClientRect.right - SplitX - MarginW, ButtonH, TRUE);

	HWND hWndStatus = GetDlgItem(hWnd, StatusCtlId);
	MoveWindow(hWndStatus, MarginW, MarginH, SplitX - (MarginW * 2), Params->ScaleY, TRUE);

	HWND hWndProgress = GetDlgItem(hWnd, ProgressCtlId);
	MoveWindow(hWndProgress, MarginW, MarginH + (Params->ScaleY * 3) / 2, SplitX - (MarginW * 2), (Params->ScaleY + 1) / 2, TRUE);

	HWND hWndLogOutput = GetDlgItem(hWnd, LogOutputCtlId);
	MoveWindow(hWndLogOutput, MarginW, SplitY, ClientRect.right - MarginW * 2, ClientRect.bottom - SplitY - MarginH, TRUE);
}

#include "Windows/HideWindowsPlatformTypes.h"
