// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FeedbackContext.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <CommCtrl.h>


/**
 * Feedback context implementation for windows.
 */
class FWindowsNativeFeedbackContext : public FFeedbackContext
{
public:
	// Constructor.
	FWindowsNativeFeedbackContext();
	virtual ~FWindowsNativeFeedbackContext();

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;
	virtual void SerializeRecord(const UE::FLogRecord& Record) override;

	virtual bool ReceivedUserCancel() override;
	virtual void StartSlowTask( const FText& Task, bool bShouldShowCancelButton=false ) override;
	virtual void FinalizeSlowTask( ) override;
	virtual void ProgressReported( const float TotalProgressInterp, FText DisplayMessage ) override;

	FContextSupplier* GetContext() const;
	void SetContext( FContextSupplier* InSupplier );

private:
	struct FWindowParams
	{
		FWindowsNativeFeedbackContext* Context;
		int32 ScaleX;
		int32 ScaleY;
		int32 StandardW;
		int32 StandardH;
		bool bLogVisible;
	};

	static const uint16 StatusCtlId = 200;
	static const uint16 ProgressCtlId = 201;
	static const uint16 ShowLogCtlId = 202;
	static const uint16 LogOutputCtlId = 203;

	FContextSupplier* Context;
	HANDLE hThread;
	HANDLE hCloseEvent;
	HANDLE hUpdateEvent;
	FCriticalSection CriticalSection;
	FString Status;
	float Progress;
	FString LogOutput;
	bool bReceivedUserCancel;
	bool bShowCancelButton;

	void CreateSlowTaskWindow(const FText &InStatus, bool bInShowCancelButton);
	void DestroySlowTaskWindow();

	static DWORD WINAPI SlowTaskThreadProc(void *Params);
	static LRESULT CALLBACK SlowTaskWindowProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam);
	static void LayoutControls(HWND hWnd, const FWindowParams* Params);
};

#include "Windows/HideWindowsPlatformTypes.h"
