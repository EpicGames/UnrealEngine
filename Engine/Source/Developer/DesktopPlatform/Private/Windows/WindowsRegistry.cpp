// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsRegistry.h"
#include "Windows/AllowWindowsPlatformTypes.h"

////////////////////////////////////////////////////////////////////

FRegistryValue::FRegistryValue()
{
	Type = 0;
}

void FRegistryValue::Set(const FString &NewValue)
{
	Type = REG_SZ;
	Data.Reset();
	Data.Append((const uint8*)*NewValue, (NewValue.Len() + 1) * sizeof(TCHAR));
}

void FRegistryValue::Set(uint32 NewValue)
{
	Type = REG_DWORD;
	Data.Reset();
	Data.Append((const uint8*)&NewValue, sizeof(uint32));
}

bool FRegistryValue::Read(HKEY hKey, const FString &Name)
{
	// Read the size and type
	DWORD ValueType;
	DWORD ValueDataLength = 0;
	if (RegQueryValueEx(hKey, *Name, NULL, &ValueType, NULL, &ValueDataLength) != ERROR_SUCCESS)
	{
		return false;
	}
	Type = (uint32)ValueType;

	// Read the value data
	Data.SetNum(ValueDataLength);
	if (RegQueryValueEx(hKey, *Name, NULL, NULL, Data.GetData(), &ValueDataLength) != ERROR_SUCCESS || ValueDataLength != Data.Num())
	{
		return false;
	}

	// Otherwise success
	return true;
}

bool FRegistryValue::Write(HKEY hKey, const FString &Name) const
{
	return (RegSetValueEx(hKey, *Name, 0, Type, Data.GetData(), (DWORD)Data.Num()) == ERROR_SUCCESS);
}

bool FRegistryValue::IsUpToDate(HKEY hKey, const FString &Name) const
{
	FRegistryValue Other;
	return Other.Read(hKey, Name) && Other.Type == Type && Other.Data == Data;
}

////////////////////////////////////////////////////////////////////

FRegistryKey::FRegistryKey()
{
}

FRegistryKey::~FRegistryKey()
{
	for (TMap<FString, FRegistryKey*>::TIterator Iter(Keys); Iter; ++Iter)
	{
		delete Iter.Value();
	}
	for (TMap<FString, FRegistryValue*>::TIterator Iter(Values); Iter; ++Iter)
	{
		delete Iter.Value();
	}
}

FRegistryKey *FRegistryKey::FindOrAddKey(const FString &Name)
{
	FRegistryKey *&Key = Keys.FindOrAdd(Name);
	if (Key == NULL)
	{
		Key = new FRegistryKey();
	}
	return Key;
}

FRegistryValue *FRegistryKey::FindOrAddValue(const FString &Name)
{
	FRegistryValue *&Value = Values.FindOrAdd(Name);
	if (Value == NULL)
	{
		Value = new FRegistryValue();
	}
	return Value;
}

void FRegistryKey::SetValue(const FString &Name, const FString &NewValue)
{
	FindOrAddValue(Name)->Set(NewValue);
}

void FRegistryKey::SetValue(const FString &Name, uint32 NewValue)
{
	FindOrAddValue(Name)->Set(NewValue);
}

bool FRegistryKey::Read(HKEY hKey)
{
	// Enumerate all the child key names
	TArray<FString> KeyNames;
	EnumerateRegistryKeys(hKey, KeyNames);

	// Read all the child keys
	for (int32 Idx = 0; Idx < KeyNames.Num(); Idx++)
	{
		FRegistryKey *Key = FindOrAddKey(KeyNames[Idx]);
		if (!Key->Read(hKey, KeyNames[Idx]))
		{
			return false;
		}
	}

	// Enumerate all the child values
	TArray<FString> ValueNames;
	EnumerateRegistryValues(hKey, ValueNames);

	// Read all the values
	for (int32 Idx = 0; Idx < ValueNames.Num(); Idx++)
	{
		FRegistryValue *Value = FindOrAddValue(ValueNames[Idx]);
		if (!Value->Read(hKey, ValueNames[Idx]))
		{
			return false;
		}
	}

	return true;
}

bool FRegistryKey::Read(HKEY hRootKey, const FString &Path)
{
	bool bRes = false;

	HKEY hKey;
	if (RegOpenKeyEx(hRootKey, *Path, 0, KEY_READ, &hKey) == ERROR_SUCCESS)
	{
		bRes = Read(hKey);
		RegCloseKey(hKey);
	}

	return bRes;
}

bool FRegistryKey::Write(HKEY hKey, bool bRemoveDifferences) const
{
	// Remove all the differences from the current content
	if (bRemoveDifferences)
	{
		// Get all the existing value names
		TArray<FString> ValueNames;
		EnumerateRegistryValues(hKey, ValueNames);

		// Remove any values that don't exist any more
		for (int32 Idx = 0; Idx < ValueNames.Num(); Idx++)
		{
			if (!Values.Contains(ValueNames[Idx]) && RegDeleteValue(hKey, *ValueNames[Idx]) != ERROR_SUCCESS)
			{
				return false;
			}
		}

		// Get all the existing key names
		TArray<FString> KeyNames;
		EnumerateRegistryKeys(hKey, KeyNames);

		// Remove any keys that don't exist any more
		for (int32 Idx = 0; Idx < KeyNames.Num(); Idx++)
		{
			if (!Keys.Contains(KeyNames[Idx]) && RegDeleteTree(hKey, *KeyNames[Idx]) != ERROR_SUCCESS)
			{
				return false;
			}
		}
	}

	// Write all the child keys
	for (TMap<FString, FRegistryKey*>::TConstIterator ChildKeyIter(Keys); ChildKeyIter; ++ChildKeyIter)
	{
		if (!ChildKeyIter.Value()->Write(hKey, ChildKeyIter.Key(), bRemoveDifferences))
		{
			return false;
		}
	}

	// Write all the child values
	for (TMap<FString, FRegistryValue*>::TConstIterator ChildValueIter(Values); ChildValueIter; ++ChildValueIter)
	{
		if (RegSetValueEx(hKey, *ChildValueIter.Key(), 0, ChildValueIter.Value()->Type, ChildValueIter.Value()->Data.GetData(), (DWORD)ChildValueIter.Value()->Data.Num()) != ERROR_SUCCESS)
		{
			return false;
		}
	}
	return true;
}

bool FRegistryKey::Write(HKEY hRootKey, const FString &Path, bool bRemoveDifferences) const
{
	bool bRes = false;

	HKEY hKey;
	if (RegCreateKeyEx(hRootKey, *Path, 0, NULL, 0, KEY_READ | KEY_WRITE, NULL, &hKey, NULL) == ERROR_SUCCESS)
	{
		bRes = Write(hKey, bRemoveDifferences);
		RegCloseKey(hKey);
	}

	return bRes;
}

bool FRegistryKey::IsUpToDate(HKEY hKey, bool bRemoveDifferences) const
{
	// Remove all the differences from the current content
	if (bRemoveDifferences)
	{
		// Get all the existing value names
		TArray<FString> ValueNames;
		EnumerateRegistryValues(hKey, ValueNames);

		// Check there aren't any extra values
		for (int32 Idx = 0; Idx < ValueNames.Num(); Idx++)
		{
			if (!Values.Contains(ValueNames[Idx]))
			{
				return false;
			}
		}

		// Get all the existing key names
		TArray<FString> KeyNames;
		EnumerateRegistryKeys(hKey, KeyNames);

		// Check there aren't any extra keys
		for (int32 Idx = 0; Idx < KeyNames.Num(); Idx++)
		{
			if (!Keys.Contains(KeyNames[Idx]))
			{
				return false;
			}
		}
	}

	// Write all the child keys
	for (TMap<FString, FRegistryKey*>::TConstIterator Iter(Keys); Iter; ++Iter)
	{
		if (!Iter.Value()->IsUpToDate(hKey, Iter.Key(), bRemoveDifferences))
		{
			return false;
		}
	}

	// Write all the child values
	for (TMap<FString, FRegistryValue*>::TConstIterator Iter(Values); Iter; ++Iter)
	{
		if (!Iter.Value()->IsUpToDate(hKey, Iter.Key()))
		{
			return false;
		}
	}
	return true;
}

bool FRegistryKey::IsUpToDate(HKEY hRootKey, const FString &Path, bool bRemoveDifferences) const
{
	bool bRes = false;

	HKEY hKey;
	if (RegOpenKeyEx(hRootKey, *Path, 0, KEY_READ, &hKey) == ERROR_SUCCESS)
	{
		bRes = IsUpToDate(hKey, bRemoveDifferences);
		RegCloseKey(hKey);
	}

	return bRes;
}

////////////////////////////////////////////////////////////////////

FRegistryRootedKey::FRegistryRootedKey(HKEY hInRootKey, const FString &InPath)
{
	hRootKey = hInRootKey;
	Path = InPath;
}

bool FRegistryRootedKey::Exists() const
{
	bool bRes = false;

	HKEY hKey;
	if (RegOpenKeyEx(hRootKey, *Path, 0, KEY_READ, &hKey) == ERROR_SUCCESS)
	{
		RegCloseKey(hKey);
		bRes = true;
	}

	return bRes;
}

bool FRegistryRootedKey::Write(bool bRemoveDifferences) const
{
	bool bRes = false;
	if (Key)
	{
		bRes = Key->Write(hRootKey, Path, bRemoveDifferences);
	}
	else
	{
		bRes = (!Exists() || RegDeleteKeyEx(hRootKey, *Path, 0, 0) == ERROR_SUCCESS);
	}
	return bRes;
}

bool FRegistryRootedKey::IsUpToDate(bool bRemoveDifferences) const
{
	bool bRes = false;
	if (Key)
	{
		bRes = Key->IsUpToDate(hRootKey, Path, bRemoveDifferences);
	}
	else
	{
		bRes = !Exists();
	}
	return bRes;
}

////////////////////////////////////////////////////////////////////

bool EnumerateRegistryKeys(HKEY hKey, TArray<FString> &OutNames)
{
	for (DWORD Index = 0;; Index++)
	{
		// Query the next key name
		TCHAR KeyName[256];
		DWORD KeyNameLength = sizeof(KeyName) / sizeof(KeyName[0]);

		LONG Result = RegEnumKeyEx(hKey, Index, KeyName, &KeyNameLength, NULL, NULL, NULL, NULL);
		if (Result == ERROR_NO_MORE_ITEMS)
		{
			break;
		}
		else if (Result != ERROR_SUCCESS)
		{
			return false;
		}

		// Add it to the array
		OutNames.Add(KeyName);
	}
	return true;
}

bool EnumerateRegistryValues(HKEY hKey, TArray<FString> &OutNames)
{
	for (DWORD Index = 0;; Index++)
	{
		// Query the value
		wchar_t ValueName[256];
		DWORD ValueNameLength = sizeof(ValueName) / sizeof(ValueName[0]);

		LONG Result = RegEnumValue(hKey, Index, ValueName, &ValueNameLength, NULL, NULL, NULL, NULL);
		if (Result == ERROR_NO_MORE_ITEMS)
		{
			break;
		}
		else if (Result != ERROR_SUCCESS)
		{
			return false;
		}

		// Add it to the array
		OutNames.Add(ValueName);
	}
	return true;
}

#include "Windows/HideWindowsPlatformTypes.h"
