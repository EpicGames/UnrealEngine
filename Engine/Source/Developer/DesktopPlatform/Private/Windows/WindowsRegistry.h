// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Windows/WindowsHWrapper.h"
#include "Templates/UniquePtr.h"

struct FRegistryValue
{
	uint32 Type;
	TArray<uint8> Data;

	FRegistryValue();

	void Set(const FString &NewValue);
	void Set(uint32 NewValue);

	bool Read(HKEY hKey, const FString &Name);
	bool Write(HKEY hKey, const FString &Name) const;
	bool IsUpToDate(HKEY hKey, const FString &Name) const;
};

struct FRegistryKey : FNoncopyable
{
	TMap<FString, FRegistryKey*> Keys;
	TMap<FString, FRegistryValue*> Values;

	FRegistryKey();
	~FRegistryKey();

	FRegistryKey *FindOrAddKey(const FString &Name);
	FRegistryValue *FindOrAddValue(const FString &Name);

	void SetValue(const FString &Name, const FString &Value);
	void SetValue(const FString &Name, uint32 Value);

	bool Read(HKEY hKey);
	bool Read(HKEY hKey, const FString &Path);

	bool Write(HKEY hKey, bool bRemoveDifferences) const;
	bool Write(HKEY hKey, const FString &Path, bool bRemoveDifferences) const;

	bool IsUpToDate(HKEY hKey, bool bRemoveDifferences) const;
	bool IsUpToDate(HKEY hKey, const FString &Path, bool bRemoveDifferences) const;
};

struct FRegistryRootedKey
{
	HKEY hRootKey;
	FString Path;
	TUniquePtr<FRegistryKey> Key;

	FRegistryRootedKey(HKEY hInKeyRoot, const FString &InPath);

	bool Exists() const;

	bool Read();
	bool Write(bool bRemoveDifferences) const;
	bool IsUpToDate(bool bRemoveDifferences) const;
};

bool EnumerateRegistryKeys(HKEY hKey, TArray<FString> &OutNames);
bool EnumerateRegistryValues(HKEY hKey, TArray<FString> &OutNames);

