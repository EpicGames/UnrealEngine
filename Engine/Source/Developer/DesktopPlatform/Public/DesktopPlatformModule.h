// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDesktopPlatform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IDesktopPlatform;

class FDesktopPlatformModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	static IDesktopPlatform* Get()
	{
		FDesktopPlatformModule& DesktopPlatformModule = FModuleManager::Get().LoadModuleChecked<FDesktopPlatformModule>("DesktopPlatform");
		return DesktopPlatformModule.GetSingleton();
	}

	static IDesktopPlatform* TryGet()
	{
		if (FDesktopPlatformModule* DesktopPlatformModule = FModuleManager::Get().LoadModulePtr<FDesktopPlatformModule>("DesktopPlatform"))
		{
			return DesktopPlatformModule->GetSingleton();
		}
		return nullptr;
	}

private:
	virtual IDesktopPlatform* GetSingleton() const { return DesktopPlatform; }

	IDesktopPlatform* DesktopPlatform;
};
