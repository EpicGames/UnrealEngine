// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CompilationResult.h"
#include "HAL/PlatformProcess.h"

class FEngineVersion;

namespace EFileDialogFlags
{
	enum Type
	{
		None					= 0x00, // No flags
		Multiple				= 0x01  // Allow multiple file selections
	};
}

enum class EFontImportFlags
{
	None = 0x0,						// No flags
	EnableAntialiasing = 0x1,		// Whether the font should be antialiased or not.  Usually you should leave this enabled.
	EnableBold = 0x2,				// Whether the font should be generated in bold or not
	EnableItalic = 0x4,				// Whether the font should be generated in italics or not
	EnableUnderline = 0x8,			// Whether the font should be generated with an underline or not
	AlphaOnly = 0x10,				// Forces PF_G8 and only maintains Alpha value and discards color
	CreatePrintableOnly = 0x20,		// Skips generation of glyphs for any characters that are not considered 'printable'
	IncludeASCIIRange = 0x40,		// When specifying a range of characters and this is enabled, forces ASCII characters (0 thru 255) to be included as well
	EnableDropShadow = 0x80,		// Enables a very simple, 1-pixel, black colored drop shadow for the generated font
	EnableLegacyMode = 0x100,		// Enables legacy font import mode.  This results in lower quality antialiasing and larger glyph bounds, but may be useful when debugging problems
	UseDistanceFieldAlpha = 0x200	// Alpha channel of the font textures will store a distance field instead of a color mask
};

ENUM_CLASS_FLAGS(EFontImportFlags)

enum class EDesktopLoginInteractionLevel : uint8
{
	None,			// No user interaction permitted
	TimeLimited,	// Allow user interaction within a limited window of time (eg: 30 seconds), but don't wait indefinitely for interaction
	Interactive		// Allow user interaction with an indefinite wait if necessary
};

/**
 * When constructed leaves system wide modal mode (all windows disabled except for the OS modal window)
 * When destructed leaves this mode
 */
class FScopedSystemModalMode
{
public:
	FScopedSystemModalMode()
	{
#if WITH_EDITOR
		FCoreDelegates::PreModal.Broadcast();
#endif
	}

	~FScopedSystemModalMode()
	{
#if WITH_EDITOR
		FCoreDelegates::PostModal.Broadcast();
#endif
	}
};

/**
 * Information about a target supported by a project
 */
struct FTargetInfo
{
	FString Name;
	FString Path;
	EBuildTargetType Type;
	TOptional<bool> DefaultTarget;
};

/**
 * Interface for functionality supported by desktop platforms
 */
class IDesktopPlatform
{
public:
	/** Virtual destructor */
	virtual ~IDesktopPlatform() = default;

	/** 
	 * Opens the "open file" dialog for the platform.
	 *
	 * @param ParentWindowHandle		The native handle to the parent window for this dialog.
	 * @param DialogTitle				The text for the title of the dialog window.
	 * @param DefaultPath				The path where the file dialog will open initially.
	 * @param DefaultFile				The file that the dialog will select initially.
	 * @param FileTypes					The type filters to show in the dialog. This string should be a "|" delimited list of (Description|Extensionlist) pairs. Extensionlists are ";" delimited.
	 * @param Flags						Details about the dialog. See EFileDialogFlags.
	 * @param OutFilenames				The filenames that were selected in the dialog.
	 * @return true if files were successfully selected.
	 */
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames) = 0;

	/** 
	 * Opens the "open file" dialog for the platform.
	 *
	 * @param ParentWindowHandle		The native handle to the parent window for this dialog.
	 * @param DialogTitle				The text for the title of the dialog window.
	 * @param DefaultPath				The path where the file dialog will open initially.
	 * @param DefaultFile				The file that the dialog will select initially.
	 * @param FileTypes					The type filters to show in the dialog. This string should be a "|" delimited list of (Description|Extensionlist) pairs. Extensionlists are ";" delimited.
	 * @param Flags						Details about the dialog. See EFileDialogFlags.
	 * @param OutFilenames				The filenames that were selected in the dialog.
	 * @param OutFilterIndex			The type that was selected in the dialog.
	 * @return true if files were successfully selected.
	 */
	virtual bool OpenFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames, int32& OutFilterIndex) = 0;

	/** 
	 * Opens the "save file" dialog for the platform.
	 *
	 * @param ParentWindowHandle		The native handle to the parent window for this dialog.
	 * @param DialogTitle				The text for the title of the dialog window.
	 * @param DefaultPath				The path where the file dialog will open initially.
	 * @param DefaultFile				The file that the dialog will select initially.
	 * @param FileTypes					The type filters to show in the dialog. This string should be a "|" delimited list of (Description|Extensionlist) pairs. Extensionlists are ";" delimited.
	 * @param Flags						Details about the dialog. See EFileDialogFlags.
	 * @param OutFilenames				The filenames that were selected in the dialog.
	 * @return true if files were successfully selected.
	 */
	virtual bool SaveFileDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, const FString& DefaultFile, const FString& FileTypes, uint32 Flags, TArray<FString>& OutFilenames) = 0;

	/** 
	 * Opens the "choose folder" dialog for the platform
	 *
	 * @param ParentWindowHandle		The native handle to the parent window for this dialog.
	 * @param DialogTitle				The text for the title of the dialog window.
	 * @param DefaultPath				The path where the file dialog will open initially.
	 * @param OutFolderName				The foldername that was selected in the dialog.
	 * @return true if folder choice was successfully selected.
	 */
	virtual bool OpenDirectoryDialog(const void* ParentWindowHandle, const FString& DialogTitle, const FString& DefaultPath, FString& OutFolderName) = 0;

	/** 
	 * Opens the "choose font" dialog for the platform
	 *
	 * @param ParentWindowHandle		The native handle to the parent window for this dialog.
	 * @param OutFontName				The name of the font.
	 * @param OutHeight					The height of the font.
	 * @param OutFlags					Any special flags the font has been tagged with.
	 * @return true if font choice was successfully selected.
	 */
	virtual bool OpenFontDialog(const void* ParentWindowHandle, FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags) = 0;

	/**
	 * Returns a description for the engine with the given identifier.
	 *
	 * @return	Description for the engine identifier. Empty string if it's unknown.
	 */
	virtual FString GetEngineDescription(const FString& Identifier) = 0;

	/**
	 * Gets the identifier for the currently executing engine installation.
	 *
	 * @return	Identifier for the current engine installation. Empty string if it isn't registered.
	 */
	virtual FString GetCurrentEngineIdentifier() = 0;

	/**
	 * Registers a directory as containing an engine installation.
	 *
	 * @param	RootDir				Root directory for the engine installation.
	 * @param	OutIdentifier		Identifier which is assigned to the engine.
	 * @return true if the directory was added. OutIdentifier will be set to the assigned identifier.
	 */
	virtual bool RegisterEngineInstallation(const FString& RootDir, FString& OutIdentifier) = 0;

	/**
	 * Enumerates all the registered engine installations.
	 *
	 * @param	OutInstallations	Map of identifier/root-directory pairs for all known installations. Identifiers are typically
	 *								version strings for canonical UE releases or GUID strings for GitHub releases.
	 */
	virtual void EnumerateEngineInstallations(TMap<FString, FString>& OutInstallations) = 0;

	/**
	 * Enumerates all the registered binary engine installations.
	 *
	 * @param	OutInstallations	Map of identifier/root-directory pairs for all known binary installations.
	 */
	virtual void EnumerateLauncherEngineInstallations(TMap<FString, FString>& OutInstallations) = 0;

	/**
	 * Enumerates all the samples installed by the launcher.
	 *
	 * @param	OutInstallations	Array of sample installation paths.
	 */
	virtual void EnumerateLauncherSampleInstallations(TArray<FString>& OutInstallations) = 0;

	/**
	 * Enumerates all the sample projects installed by the launcher.
	 *
	 * @param	OutFileNames		Array of sample project filenames.
	 */
	virtual void EnumerateLauncherSampleProjects(TArray<FString>& OutFileNames) = 0;

	/**
	 * Returns the root directory for the engine with the given identifier.
	 *
	 * @param	Identifier			Identifier used to refer to this installation.
	 * @param	OutRootDir			Root directory for the engine.
	 */
	virtual bool GetEngineRootDirFromIdentifier(const FString& Identifier, FString& OutRootDir) = 0;

	/**
	 * Returns the identifier for the engine with the given root directory.
	 *
	 * @param	RootDir				Root directory for the engine.
	 * @param	OutIdentifier		Identifier used to refer to this installation.
	 */
	virtual bool GetEngineIdentifierFromRootDir(const FString& RootDir, FString& OutIdentifier) = 0;

	/**
	 * Gets the identifier for the default engine. This will be the newest installed engine.
	 *
	 * @param	OutIdentifier		String to hold to the default engine's identifier
	 * @return	true if OutIdentifier is valid.
	 */
	virtual bool GetDefaultEngineIdentifier(FString& OutIdentifier) = 0;

	/**
	 * Compares two identifiers and checks whether the first is preferred to the second.
	 *
	 * @param	Identifier			First identifier
	 * @param	OtherIdentifier		Second identifier
	 * @return	true if Identifier is preferred over OtherIdentifier.
	 */
	virtual bool IsPreferredEngineIdentifier(const FString& Identifier, const FString& OtherIdentifier) = 0;

	/**
	 * Gets the root directory for the default engine installation.
	 *
	 * @param	OutRootDir			String to hold to the default engine's root directory.
	 * @return	true if OutRootDir is valid.
	 */
	virtual bool GetDefaultEngineRootDir(FString& OutRootDir) = 0;

	/**
	 * Gets the root directory for the engine's saved config files
	 *
	 * @param	Identifier			Identifier for the engine
	 * @return	The absoluate path to the engines Saved/Config directory
	 */
	virtual FString GetEngineSavedConfigDirectory(const FString& Identifier) = 0;

	/**
	 * Attempt to get the engine version from the supplied engine root directory.
	 * Tries to read the contents of Engine/Build/Build.version, but scrapes information 
	 * from the Engine/Source/Launch/Resources/Version.h if it fails.
	 *
	 * @param	RootDir				Root directory for the engine to check
	 * @param	OutVersion			Engine version obtained from identifier
	 * @return	true if the engine version was successfully obtained
	 */
	virtual bool TryGetEngineVersion(const FString& RootDir, FEngineVersion& OutVersion) = 0;

	/**
	 * Checks if the given engine identifier is for an stock engine release.
	 *
	 * @param	Identifier			Engine identifier to check
	 * @return	true if the identifier is for a binary engine release
	 */
	virtual bool IsStockEngineRelease(const FString& Identifier) = 0;

	/**
	 * Attempt to get the engine version from the supplied identifier
	 *
	 * @param	Identifier			Engine identifier to check
	 * @param	OutVersion			Engine version obtained from identifier
	 * @return	true if the engine version was successfully obtained
	 */
	virtual bool TryParseStockEngineVersion(const FString& Identifier, FEngineVersion& OutVersion) = 0;

	/**
	 * Tests whether an engine installation is a source distribution.
	 *
	 * @return	true if the engine contains source.
	 */
	virtual bool IsSourceDistribution(const FString& RootDir) = 0;

	/**
	 * Tests whether an engine installation is a perforce build.
	 *
	 * @return	true if the engine is a perforce build.
	 */
	virtual bool IsPerforceBuild(const FString& RootDir) = 0;

	/**
	 * Tests whether a root directory is a valid Unreal Engine installation
	 *
	 * @return	true if the engine is a valid installation
	 */
	virtual bool IsValidRootDirectory(const FString& RootDir) = 0;

	/**
	 * Checks that the current file associations are correct.
	 *
	 * @return	true if file associations are up to date.
	 */
	virtual bool VerifyFileAssociations() = 0;

	/**
	 * Updates file associations.
	 *
	 * @return	true if file associations were successfully updated.
	 */
	virtual bool UpdateFileAssociations() = 0;

	/**
	 * Sets the engine association for a project.
	 *
	 * @param ProjectFileName		Filename of the project to update
	 * @param Identifier			Identifier of the engine to associate it with
	 * @return true if the project was successfully updated
	 */
	virtual bool SetEngineIdentifierForProject(const FString& ProjectFileName, const FString& Identifier) = 0;

	/**
	 * Gets the engine association for a project.
	 *
	 * @param ProjectFileName		Filename of the project to update
	 * @param OutIdentifier			Identifier of the engine to associate it with
	 * @return true if OutIdentifier is set to the project's engine association
	 */
	virtual bool GetEngineIdentifierForProject(const FString& ProjectFileName, FString& OutIdentifier) = 0;

	/**
	 * Opens the given project with the appropriate editor. Tries to use the shell association.
	 *
	 * @param ProjectFileName		Filename of the project to update
	 * @return true if the project was opened successfully.
	 */
	virtual bool OpenProject(const FString& ProjectFileName) = 0;

	/**
	 * Cleans a game project. Removes the intermediate folder and binary build products.
	 *
	 * @param ProjectDir			Directory of the project to clean.
	 * @param OutFailPath			The path of the first file or directory that failed to delete, if any.
	 * @param Warn					Feedback context to use for progress updates.
	 * @return						true if the project was cleaned successfully, false if any file or directory could not be deleted.
	 */
	virtual bool CleanGameProject(const FString& ProjectDir, FString& OutFailPath, FFeedbackContext* Warn) = 0;

	/**
	 * Compiles a game project.
	 *
	 * @param RootDir				Engine root directory for the project to use.
	 * @param ProjectFileName		Filename of the project to update
	 * @param Warn					Feedback context to use for progress updates
	 * @param OutResult				The compilation result. May be null.
	 * @return true if project files were generated successfully.
	 */
	virtual bool CompileGameProject(const FString& RootDir, const FString& ProjectFileName, FFeedbackContext* Warn, ECompilationResult::Type* OutResult = nullptr) = 0;

	/**
	 * Generates project files for the given project.
	 *
	 * @param RootDir				Engine root directory for the project to use.
	 * @param ProjectFileName		Filename of the project to update
	 * @param Warn					Feedback context to use for progress updates
	 * @return true if project files were generated successfully.
	 */
	virtual bool GenerateProjectFiles(const FString& RootDir, const FString& ProjectFileName, FFeedbackContext* Warn, FString LogFilePath = FString()) = 0;

	/**
	 * Determines whether UnrealBuildTool is available
	 *
	 * @return true if UnrealBuildTool is available
	 */
	virtual bool IsUnrealBuildToolAvailable() = 0;

	/**
	 * Invokes UnrealBuildTool with the given arguments
	 *
	 * @return true if tool was invoked properly
	 */
	virtual bool InvokeUnrealBuildToolSync(const FString& InCmdLineParams, FOutputDevice& Ar, bool bSkipBuildUBT, int32& OutReturnCode, FString& OutProcOutput) = 0;

	/** 
	 * Launches UnrealBuildTool with the specified command line parameters 
	 * 
	 * @return process handle to the new UBT process
	 */
	virtual FProcHandle InvokeUnrealBuildToolAsync(const FString& InCmdLineParams, FOutputDevice& Ar, void*& OutReadPipe, void*& OutWritePipe, bool bSkipBuildUBT = false) = 0;

	/**
	 * Runs UnrealBuildTool with the given arguments.
	 *
	 * @param Description		Task description for FFeedbackContext
	 * @param RootDir			Engine root directory for the project to use.
	 * @param Arguments			Parameters for UnrealBuildTool
	 * @param Warn				Feedback context to use for progress updates
	 * @return true if the task completed successfully.
	 */
	virtual bool RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn) = 0;

	/**
	 * Runs UnrealBuildTool with the given arguments.
	 *
	 * @param Description		Task description for FFeedbackContext
	 * @param RootDir			Engine root directory for the project to use.
	 * @param Arguments			Parameters for UnrealBuildTool
	 * @param Warn				Feedback context to use for progress updates
	 * @param OutExitCode		Receives the process exit code
	 * @return true if the task completed successfully.
	 */
	virtual bool RunUnrealBuildTool(const FText& Description, const FString& RootDir, const FString& Arguments, FFeedbackContext* Warn, int32& OutExitCode) = 0;

	/**
	 * Checks if an instance of UnrealBuildTool is running.
	 *
	 * @return true if an instance of UnrealBuildTool is running.
	 */
	virtual bool IsUnrealBuildToolRunning() = 0;

	/**
	 * Gets information about the build targets supported by a particular project.
	 *
	 * @return Array of TargetInfo objects
	 */
	virtual const TArray<FTargetInfo>& GetTargetsForProject(const FString& ProjectFile) const = 0;

	/**
	 * Gets information about the build targets supported by the current project.
	 *
	 * @return Array of TargetInfo objects
	 */
	virtual const TArray<FTargetInfo>& GetTargetsForCurrentProject() const = 0;

	/**
	 * Gets the path to the user's temporary directory
	 *
	 * @return The path to the user's temporary directory
	 */
	virtual FString GetUserTempPath() = 0;

	/**
	 * Gets a feedback context which can display progress information using the native platform GUI.
	 *
	 * @return FFeedbackContext for the native GUI.
	 */
	virtual FFeedbackContext* GetNativeFeedbackContext() = 0;

	/**
	 * Finds all the projects which the engine (given by an identifier) has a record of. This includes all the recently opened projects, projects in the default project directory, and so on.
	 * 
	 * @param Identifier				Identifier for the engine 
	 * @param bIncludeNativeProjects	Flag indicating whether to include native projects.
	 * @param OutProjectFileNames		Array to receive the paths of the projects known by the engine.
	 * @return True if the enumeration was successful, false otherwise.
	 */
	virtual bool EnumerateProjectsKnownByEngine(const FString& Identifier, bool bIncludeNativeProjects, TArray<FString>& OutProjectFileNames) = 0;

	/**
	 * Gets the default folder for creating new projects.
	 * 
	 * @return Path to the folder
	 */
	virtual FString GetDefaultProjectCreationPath() = 0;

	/**
	 * Get an access token for use by OIDC compliant Identity Providers

	 * @param RootDir					Engine root directory for the project to use.
	 * @param ProjectFileName			Filename of the current project
	 * @param ProviderIdentifier		The identifier for the provider as configured in oidc-token.json
	 * @param bUnattended				True to indicate that no user interaction should be assumed
	 * @param Warn						Feedback context to use for progress updates
	 * @param OutToken					The allocated access token
	 * @param OutTokenExpiresAt			When the token expires
	 * @param bOutWasInteractiveLogin	True if the interactive login flow was used
	 * @return true if the task completed successfully.
	 */
	virtual bool GetOidcAccessToken(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) = 0;

	/**
	 * Get an access token for use by OIDC compliant Identity Providers using a remote configuration

	 * @param RootDir					Engine root directory for the project to use.
	 * @param RemoteHost				The url to the remote host to fetch the config from
	 * @param bUnattended				True to indicate that no user interaction should be assumed
	 * @param Warn						Feedback context to use for progress updates
	 * @param OutToken					The allocated access token
	 * @param OutTokenExpiresAt			When the token expires
	 * @param bOutWasInteractiveLogin	True if the interactive login flow was used
	 * @return true if the task completed successfully.
	 */
	virtual bool GetOidcAccessTokenWithRemoteConfig(const FString& RootDir, const FString& RemoteHost, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) = 0;

	/**
	 * Get an access token for use by OIDC compliant Identity Providers

	 * @param RootDir					Engine root directory for the project to use.
	 * @param ProjectFileName			Filename of the current project
	 * @param ProviderIdentifier		The identifier for the provider as configured in oidc-token.json
	 * @param bUnattended				True to indicate that no user interaction should be assumed
	 * @param Warn						Feedback context to use for progress updates
	 * @param OutToken					The allocated access token
	 * @param OutTokenExpiresAt			When the token expires
	 * @param bOutWasInteractiveLogin	True if the interactive login flow was used
	 * @return true if the task completed successfully.
	 */
	UE_DEPRECATED(5.7, "Please use overload that takes EDesktopLoginInteractionLevel instead")
	virtual bool GetOidcAccessToken(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) = 0;

	/**
	 * Get an access token for use by OIDC compliant Identity Providers using a remote configuration

	 * @param RootDir					Engine root directory for the project to use.
	 * @param RemoteHost				The url to the remote host to fetch the config from
	 * @param bUnattended				True to indicate that no user interaction should be assumed
	 * @param Warn						Feedback context to use for progress updates
	 * @param OutToken					The allocated access token
	 * @param OutTokenExpiresAt			When the token expires
	 * @param bOutWasInteractiveLogin	True if the interactive login flow was used
	 * @return true if the task completed successfully.
	 */
	UE_DEPRECATED(5.7, "Please use overload that takes EDesktopLoginInteractionLevel instead")
	virtual bool GetOidcAccessTokenWithRemoteConfig(const FString& RootDir, const FString& RemoteHost, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) = 0;

	/**
	 * Get the status of an access token for use by OIDC compliant Identity Providers

	 * @param RootDir					Engine root directory for the project to use.
	 * @param ProjectFileName			Filename of the current project
	 * @param ProviderIdentifier		The identifier for the provider as configured in oidc-token.json
	 * @param Warn						Feedback context to use for progress updates
	 * @param OutStatus					The allocated access token
	 * @return true if the task completed successfully.
	 */
	virtual bool GetOidcTokenStatus(const FString& RootDir, const FString& ProjectFileName, const FString& ProviderIdentifier, FFeedbackContext* Warn, int& OutStatus) = 0;

	/**
	 * Get the path to the OIDC token executable

	 * @param RootDir					Engine root directory for the project to use.
	 * @return The path to the OIDC token executable.
	 */
	UE_INTERNAL virtual FString GetOidcTokenExecutableFilename(const FString& RootDir) const = 0;

	/**
	 * Get the URL of the configured Horde server for the current user account.
	 *
	 * @param OutHordeUrl				URL of the Horde server
	 * @param OutHordeUrlConfigSource	Optional output parameter describing the source of the configuration.
	 * @return true if a valid URL is returned.
	 */
	virtual bool GetHordeUrl(FString& OutHordeUrl, FString* OutHordeUrlConfigSource = nullptr) = 0;

	/**
	 * Set the URL of the configured Horde server for the current user account. This will set the source of the configuration to EConfigSource::Unspecified.
	 *
	 * @param HordeUrl					URL of the Horde server
	 */
	virtual void SetHordeUrl(const FString& HordeUrl) = 0;

	/**
	 * Gets an access token for the given Horde Server.
	 *
	 * @param HordeUrl					URL of the Horde server.
	 * @param bUnattended				True to indicate that no user interaction should be assumed
	 * @param Warn						Feedback context to use for progress updates
	 * @param OutToken					The allocated access token
	 * @param OutTokenExpiresAt			When the token expires
	 * @param bOutWasInteractiveLogin	True if the interactive login flow was used
	 * @return true if the task completed successfully.
	 */
	virtual bool GetHordeAccessToken(const FString& HordeUrl, EDesktopLoginInteractionLevel InteractionLevel, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) = 0;

	/**
	 * Gets an access token for the given Horde Server.
	 *
	 * @param HordeUrl					URL of the Horde server.
	 * @param bUnattended				True to indicate that no user interaction should be assumed
	 * @param Warn						Feedback context to use for progress updates
	 * @param OutToken					The allocated access token
	 * @param OutTokenExpiresAt			When the token expires
	 * @param bOutWasInteractiveLogin	True if the interactive login flow was used
	 * @return true if the task completed successfully.
	 */
	UE_DEPRECATED(5.7, "Please use overload that takes EDesktopLoginInteractionLevel instead")
	virtual bool GetHordeAccessToken(const FString& HordeUrl, bool bUnattended, FFeedbackContext* Warn, FString& OutToken, FDateTime& OutTokenExpiresAt, bool& bOutWasInteractiveLogin) = 0;
};
