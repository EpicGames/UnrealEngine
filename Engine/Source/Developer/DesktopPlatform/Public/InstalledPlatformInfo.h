// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Optional.h"

#define UE_API DESKTOPPLATFORM_API

template <typename FuncType> class TFunctionRef;

// Forward declaration
enum class EProjectType : uint8
{
	Unknown,
	Any,
	Code,
	Content,
};

EProjectType EProjectTypeFromString(const FString& ProjectTypeName);

enum class EInstalledPlatformState
{
	/** Query whether the platform is supported */
	Supported,

	/** Query whether the platform has been downloaded */
	Downloaded,
};

/**
 * Information about a single installed platform configuration
 */
struct FInstalledPlatformConfiguration
{
	/** Build Configuration of this combination */
	EBuildConfiguration Configuration;

	/** Name of the Platform for this combination */
	FString PlatformName;

	/** Type of Platform for this combination */
	EBuildTargetType PlatformType;

	/** Name of the Architecture for this combination */
	FString Architecture;

	/** Location of a file that must exist for this combination to be valid (optional) */
	FString RequiredFile;

	/** Type of project this configuration can be used for */
	EProjectType ProjectType;

	/** Whether to display this platform as an option even if it is not valid */
	bool bCanBeDisplayed;
};

/**
 * Singleton class for accessing information about installed platform configurations
 */
class FInstalledPlatformInfo
{
public:
	/**
	 * Accessor for singleton
	 */
	static UE_API FInstalledPlatformInfo& Get();

	/**
	 * Queries whether a configuration is valid for any available platform
	 */
	UE_API bool IsValidConfiguration(const EBuildConfiguration Configuration, EProjectType ProjectType = EProjectType::Any) const;

	/**
	 * Queries whether a platform has any valid configurations
	 */
	UE_API bool IsValidPlatform(const FString& PlatformName, EProjectType ProjectType = EProjectType::Any) const;

	/**
	 * Queries whether a platform and configuration combination is valid
	 */
	UE_API bool IsValidPlatformAndConfiguration(const EBuildConfiguration Configuration, const FString& PlatformName, EProjectType ProjectType = EProjectType::Any) const;

	/**
	 * Queries whether a platform can be displayed as an option, even if it's not supported for the specified project type
	 */
	UE_API bool CanDisplayPlatform(const FString& PlatformName, EProjectType ProjectType) const;

	/**
	 * Queries whether a target type is valid for any configuration
	 */
	UE_API bool IsValidTargetType(EBuildTargetType TargetType) const;

	/** Determines whether the given target type is supported
	 * @param TargetType The target type being built
	 * @param Platform The platform being built
	 * @param Configuration The configuration being built
	 * @param ProjectType The project type required
	 * @param State State of the given platform support
	 * @return True if the target can be built
	 */
	UE_API bool IsValid(TOptional<EBuildTargetType> TargetType, TOptional<FString> Platform, TOptional<EBuildConfiguration> Configuration, EProjectType ProjectType, EInstalledPlatformState State) const;

	/**
	 * Queries whether a platform architecture is valid for any configuration
	 * @param PlatformName Name of the platform's binary folder (eg. Win64, Android)
	 * @param Architecture Either a full architecture name or a partial substring for CPU/GPU combinations (eg. "-armv7", "-es2")
	 */
	UE_API bool IsValidPlatformArchitecture(const FString& PlatformName, const FString& Architecture) const;

	/**
	 * Queries whether a platform has any missing required files
	 */
	UE_API bool IsPlatformMissingRequiredFile(const FString& PlatformName) const;

	/**
	 * Attempts to open the Launcher to the Installer options so that additional platforms can be downloaded
	 *
	 * @return false if the engine is not a stock release, user cancels action or launcher fails to load
	 */
	static UE_API bool OpenInstallerOptions();

private:
	/**
	 * Constructor
	 */
	FInstalledPlatformInfo();

	/**
	 * Parse platform configuration info from a config file entry
	 */
	void ParsePlatformConfiguration(FString PlatformConfiguration);

	/**
	 * Given a filter function, checks whether any configuration passes that filter and has required file
	 */
	bool ContainsValidConfiguration(TFunctionRef<bool(const FInstalledPlatformConfiguration)> ConfigFilter) const;

	/**
	 * Given a filter function, checks whether any configuration passes that filter
	 * Doesn't check whether required file exists, so that we can find platforms that can be optionally installed
	 */
	bool ContainsMatchingConfiguration(TFunctionRef<bool(const FInstalledPlatformConfiguration)> ConfigFilter) const;

	/** List of installed platform configuration combinations */
	TArray<FInstalledPlatformConfiguration> InstalledPlatformConfigurations;
};

#undef UE_API
