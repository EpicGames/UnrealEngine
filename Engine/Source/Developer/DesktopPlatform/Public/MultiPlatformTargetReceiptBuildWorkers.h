// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "TargetReceiptBuildWorker.h"
#include "Templates/TypeCompatibleBytes.h"

#define UE_API DESKTOPPLATFORM_API

/**
 * Globally registers a UE::DerivedData::IBuildWorkerFactory instance for each platform that build workers can be supported.
 * Users should include a $(Platform) variable in their file path to indicate that this worker receipt can exist for any platform.
 * If the receipt path does not include a $(Platform) variable, then no attempt will be made to find variations of it for other platforms.
 */
class FMultiPlatformTargetReceiptBuildWorkers final
{
public:
	UE_API FMultiPlatformTargetReceiptBuildWorkers(FStringView TargetReceiptFilePath);
	UE_API ~FMultiPlatformTargetReceiptBuildWorkers();

	FMultiPlatformTargetReceiptBuildWorkers(const FMultiPlatformTargetReceiptBuildWorkers&) = delete;
	FMultiPlatformTargetReceiptBuildWorkers& operator=(const FMultiPlatformTargetReceiptBuildWorkers&) = delete;

private:
	enum ESupportedPlatform : uint8
	{
		SupportedPlatform_Win64 = 0,
		SupportedPlatform_Mac,
		SupportedPlatform_Linux,

		SupportedPlatform_MAX
	};
	TTypeCompatibleBytes<FTargetReceiptBuildWorker> PlatformSpecificWorkerFactories[SupportedPlatform_MAX];
	bool bAllPlatformsInitialized = false;
};

#undef UE_API
