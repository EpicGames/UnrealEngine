// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Internationalization/Text.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "UObject/NameTypes.h"

#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA

namespace PlatformInfo
{
	/** Flags describing platform variants */
	namespace EPlatformFlags
	{
		typedef uint8 Flags;
		enum Flag
		{
			/** Nothing of interest */
			None = 0,

			/** The flavor generates different output when building (eg, 32 or 64-bit) */
			BuildFlavor = 1 << 0,

			/** The flavor generates different output when cooking (eg, ETC2 or ASTC texture format) */
			CookFlavor = 1 << 1,
		};
	}

	/** Flavor types used when filtering the platforms based upon their flags */
	enum class EPlatformFilter : uint8
	{
		/** Include all platform types */
		All,

		/** Include only build flavors */
		BuildFlavor,

		/** Include only cook flavors */
		CookFlavor,
	};

	/** Information about a given platform */
	struct FTargetPlatformInfo
	{
		DESKTOPPLATFORM_API FTargetPlatformInfo(const FString& InIniPlatformName, EBuildTargetType InType, const FString& InCookFlavor);

		/** Name of the Info object as well as the ITargetPlatform that this Info describes */
		FName Name;

		/** Platform flavor, eg "ETC2" for "Android_ETC2" */
		FName PlatformFlavor;

		/** The friendly (and localized) display name of this platform */
		FText DisplayName;

		/** Type of this platform */
		EBuildTargetType PlatformType;

		/** Flags for this platform */
		EPlatformFlags::Flags PlatformFlags;

		/** Additional argument string data to append to UAT commands relating to this platform */
		FString UATCommandLine;

		/** Name of this platform when loading INI files (and finding DataDrivenPlatformInfo) */
		FName IniPlatformName;

		/** For flavors, this points to the vanilla (parent) object - for vanilla objects, this points to itself so ->VanillaInfo can be used without checking for null */
		FTargetPlatformInfo* VanillaInfo;

		/** For vanilla objects, this contains the flavors (children) */
		TArray<const FTargetPlatformInfo*> Flavors;

		/** Cached pointer to the DDPI */
		const FDataDrivenPlatformInfo* DataDrivenPlatformInfo;

		/** Returns true if this platform is vanilla */
		FORCEINLINE bool IsVanilla() const
		{
			return PlatformFlavor.IsNone();
		}

		/** Returns true if this platform is a flavor */
		FORCEINLINE bool IsFlavor() const
		{
			return !PlatformFlavor.IsNone();
		}

		// convenience function
		FName GetIconStyleName(const EPlatformIconSize InIconSize) const
		{
			return DataDrivenPlatformInfo->GetIconStyleName(InIconSize);
		}

	};

	/**
	 * Try and find the information for the given platform
	 * @param InPlatformName - The name of the platform to find
	 * @return The platform info if the platform was found, null otherwise
	 */
	DESKTOPPLATFORM_API const FTargetPlatformInfo* FindPlatformInfo(const FName& InPlatformName);

	/**
	 * Try and find the vanilla information for the given platform
	 * @param InPlatformName - The name of the platform to find (can be a flavor, but you'll still get back the vanilla platform)
	 * @return The platform info if the platform was found, null otherwise
	 */
	DESKTOPPLATFORM_API const FTargetPlatformInfo* FindVanillaPlatformInfo(const FName& InPlatformName);

	/**
	 * Get an array of all the platforms we know about
	 * @return The pointer to the start of the platform array
	 */
	DESKTOPPLATFORM_API const TArray<FTargetPlatformInfo*>& GetPlatformInfoArray();

	/**
	 * Get an array of only the vanilla platforms
	 * @return The pointer to the start of the platform array
	 */
	DESKTOPPLATFORM_API const TArray<FTargetPlatformInfo*>& GetVanillaPlatformInfoArray();

	/**
	* Update the display name for a platform
	* @param InPlatformName - The platform to update
	* @param InDisplayName - The new display name
	*/
	DESKTOPPLATFORM_API void UpdatePlatformDisplayName(FString InPlatformName, FText InDisplayName);

	/**
	* Returns a list of all defined Platform Groups, excluding None.
    * Used to to present a list in the Per-Platform Properties UI.
	* @return An array of FNames.
	*/
	DESKTOPPLATFORM_API const TArray<FName>& GetAllPlatformGroupNames();

	/**
	* Returns a list of all defined Platform, in vanilla form.
    * Used to to present a list in the Per-Platform Properties UI.
	* @return An array of FNames.
	*/
	DESKTOPPLATFORM_API const TArray<FName>& GetAllVanillaPlatformNames();
}

#endif
