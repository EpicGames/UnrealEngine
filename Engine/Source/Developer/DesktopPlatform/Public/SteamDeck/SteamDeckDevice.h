// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Common/TargetPlatformBase not part of additional include paths

#include "CoreMinimal.h"
#include "Common/TargetPlatformBase.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/MemoryLayout.h"
#include "Logging/LogMacros.h"

// Extend from either WIndows or Linux device, since there will be a lot of shared functionality
template<class ParentDeviceClass>
class TSteamDeckDevice : public ParentDeviceClass
{
public:
	TSteamDeckDevice(FString InIpAddr, FString InDeviceName, FString InUserName, FString InPassword, const ITargetPlatformControls& InTargetPlatformControls, const TCHAR* InRuntimeOSName)
		: ParentDeviceClass(InTargetPlatformControls)
		, IpAddr(InIpAddr)
		, UserName(InUserName)
		, Password(InPassword)
		, RuntimeOSName(InRuntimeOSName)
	{
		DeviceName = FString::Printf(TEXT("%s (%s)"), *InDeviceName, InRuntimeOSName);
	}

	virtual FString GetName() const override
	{
		return DeviceName;
	}

	virtual FTargetDeviceId GetId() const override
	{
		return FTargetDeviceId(this->TargetPlatformControls.PlatformName(), IpAddr);
	}

	virtual FString GetOperatingSystemName() override
	{
		return FString::Printf(TEXT("SteamOS (%s)"), *RuntimeOSName);
	}

	virtual bool GetUserCredentials(FString& OutUserName, FString& OutUserPassword) override
	{
		OutUserName = UserName;
		OutUserPassword = Password;

		return true;
	}

	static TArray<ITargetDevicePtr> DiscoverDevices(const ITargetPlatformControls& TargetPlatformControls, const TCHAR* RuntimeOSName)
	{
		TArray<FString> EngineIniSteamDeckDevices;

		// Expected ini format: +SteamDeckDevice=(IpAddr=10.1.33.19,Name=MySteamDeck,UserName=deck,Password=password)
		GConfig->GetArray(TEXT("SteamDeck"), TEXT("SteamDeckDevice"), EngineIniSteamDeckDevices, GEngineIni);

		TArray<ITargetDevicePtr> SteamDevices;
		for (const FString& Device : EngineIniSteamDeckDevices)
		{
			FString IpAddr;
			FString Name;
			FString ConfigUserName;
			FString ConfigPassword;
			
			// As of SteamOS version 3.1, "deck" is the required UserName to be used when making a remote connection
			// to the device. Eventually it will be configurable, so we will allow users to set it via the config. 
			static const FString DefaultUserName = TEXT("deck");
			
			if (!FParse::Value(*Device, TEXT("IpAddr="), IpAddr))
			{
				UE_LOGF(LogHAL, Error, "You must specify the 'IpAddr' field to connect to a SteamDeck!");
				continue;
			}
			
			// Name is not required, if not set use the IpAddr. This is what is displayed in the Editor
			if (!FParse::Value(*Device, TEXT("Name="), Name))
			{
				static const FString DefaultNamePrefix = "[SteamDeck] ";
				Name = DefaultNamePrefix + IpAddr;
			}

			// If the user has not specified a UserName in the config, fallback to the default user name for the SteamDeck
			if (!FParse::Value(*Device, TEXT("UserName="), ConfigUserName))
			{
				ConfigUserName = DefaultUserName;
			}

			// If the user has not specified a Password in the config, use an rsa key that is part of the SteamOS Devkit Client
			if (!FParse::Value(*Device, TEXT("Password="), ConfigPassword))
			{
				ConfigPassword = FString();
			}

			SteamDevices.Add(MakeShareable(new TSteamDeckDevice<ParentDeviceClass>(IpAddr, Name, ConfigUserName, ConfigPassword, TargetPlatformControls, RuntimeOSName)));
		}

		return SteamDevices;
	}

private:
	FString IpAddr;
	FString DeviceName;
	FString UserName;
	FString Password;
	FString RuntimeOSName;
};
