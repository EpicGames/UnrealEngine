// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Optional.h"
#include "Modules/BuildVersion.h"

#define UE_API DESKTOPPLATFORM_API

/**
 * Record of a file that was created as part of the build process
 */
struct FBuildProduct
{
	/** Path to the file. */
	FString Path;

	/** Type of the build product. */
	FString Type;
};

/** 
 * Information about a file which is required by the target at runtime, and must be moved around with it.
 */
struct FRuntimeDependency
{
	/** The file that should be staged. Should use $(EngineDir) and $(ProjectDir) variables as a root, so that the target can be relocated to different machines. */
	FString Path;

	/** How to stage this file. */
	FString Type;
};

/**
 * Arbitrary property name/value which metadata from the build scripts can be passed on to downstream tasks
 */
struct FReceiptProperty
{	
	/** Property name */
	FString Name;

	/** Value of the property */
	FString Value;
};

/**
 * Stores information about a compiled target. Mirror of the TargetReceipt class generated by UBT.
 */
struct FTargetReceipt
{
	/** Path to the project file for this target */
	FString ProjectFile;

	/** The project directory */
	FString ProjectDir;

	/** The name of this target */
	FString TargetName;

	/** Which platform the target is compiled for */
	FString Platform;

	/** Which platform the target is compiled for */
	FString Architecture;

	/** What build settings version the target is using */
	FString BuildSettingsVersion;

	/** What build environment the target is using */
	FString TargetBuildEnvironment;

	/** Which configuration this target is compiled in */
	EBuildConfiguration Configuration;

	/** The type of the target */
	EBuildTargetType TargetType;

	/** The version information for this target */
	FBuildVersion Version;

	/**
	 * The executable to launch for this target.
	 * (NOTE: If the LaunchCmd field exists, but Launch does not, this will actually reflect LaunchCmd!)
	 */
	FString Launch;

	/** The console subsystem executable for this target */
	TOptional<FString> LaunchCmd;

	/** The build products which are part of this target */
	TArray<FBuildProduct> BuildProducts;

	/** All the runtime dependencies that this target relies on */
	TArray<FRuntimeDependency> RuntimeDependencies;

	/** All plugins that were either enabled or disabled via the target rules. */
	TMap<FString, bool> PluginNameToEnabledState;

	/** All plugins which are part of this target. */
	TArray<FString> BuildPlugins;

	/** Additional build properties passed through from the module rules */
	TArray<FReceiptProperty> AdditionalProperties;

	/**
	 * Read a target receipt from disk
	 *
	 * @param FileName The file to read from
	 * @param bExpandVariables Whether variables should be expanded in paths or not
	 * @return True if the file was read successfully
	 */
	UE_API bool Read(const FString& FileName, bool bExpandVariables = true);

	/**
	 * Compares the running executable's path to the `Launch` and/or `LaunchCmd` paths.
	 * 
	 * @param bCheckLaunchField Return true if the current executable is the one referred to by the `Launch` field.
	 * @param bCheckLaunchCmdField Return true if the current executable is the one referred to by the `LaunchCmd` field.
	 * @return True if either specified check matches; otherwise, false.
	 */
	UE_API bool LaunchesCurrentExecutable(bool bCheckLaunchField = true, bool bCheckLaunchCmdField = true) const;

	/**
	 * Gets the default path for a target receipt
	 *
	 * @param BaseDir Base directory for the target being built; either the project directory or engine directory.
	 * @param TargetName The target being built
	 * @param Platform The target platform
	 * @param Configuration The target configuration
	 * @param BuildArchitecture The architecture being built
	 * @return Path to the receipt for this target
	 */
	static UE_API FString GetDefaultPath(const TCHAR* BaseDir, const TCHAR* TargetName, const TCHAR* Platform, EBuildConfiguration Configuration, const TCHAR* BuildArchitecture);

private:
	/**
	 * Expands the $(EngineDir) and $(ProjectDir) variables within a string
	 *
	 * @param Path Path to expand variables within
	 */
	void ExpandVariables(FString& Path);
};

#undef UE_API
