// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Misc/OptionalFwd.h"
#include "Templates/FunctionWithContext.h"
#include "Templates/PimplPtr.h"

#define UE_API DESKTOPPLATFORM_API

class FTargetReceiptBuildWorkerFactory;

namespace UE::DerivedData { class FBuildWorker; }
namespace UE::DerivedData { class IBuildWorkerRegistry; }

struct FTargetReceiptBuildWorkerParams
{
	/** When provided, override the default saved builder worker package path with this path. */
	FStringView WorkerPackagePath;
	/** When provided, called periodically to check if build worker creation should abort early. */
	UE::TFunctionWithContext<bool ()> ShouldAbort = nullptr;
	/** When true, skip timestamp checks and unconditionally create the build worker package. */
	bool bAlwaysBuild = false;
	/** When true, include symbol files in the build worker package. */
	bool bIncludeSymbols = false;
	/** When true, skips compression of the build worker package attachments. */
	bool bSkipCompression = false;
};

/**
 * Creates a build worker package from the target receipt.
 *
 * @param TargetReceiptPath   Path to load the target receipt from.
 * @param Registry   Registry provided to avoid DesktopPlatform requiring a link dependency on DerivedDataCache.
 * @param Params   Optional parameters to configure build worker package creation.
 * @return A build worker for a package saved to the path, or unset on error.
 */
[[nodiscard]] UE_API TOptional<UE::DerivedData::FBuildWorker> CreateBuildWorkerFromTargetReceipt(
	FStringView TargetReceiptPath,
	UE::DerivedData::IBuildWorkerRegistry& Registry,
	const FTargetReceiptBuildWorkerParams& Params = {});

/**
 * Globally registers a UE::DerivedData::IBuildWorkerFactory instance that runs an executable built by UnrealBuildTool.
 * UnrealBuildTool provides the executable information through a TargetReceipt file.
 */
class FTargetReceiptBuildWorker final
{
public:
	UE_API FTargetReceiptBuildWorker(FStringView TargetReceiptFilePath);
	UE_API ~FTargetReceiptBuildWorker();

	FTargetReceiptBuildWorker(const FTargetReceiptBuildWorker&) = delete;
	FTargetReceiptBuildWorker& operator=(const FTargetReceiptBuildWorker&) = delete;

private:
	TPimplPtr<FTargetReceiptBuildWorkerFactory> Factory;
};

#undef UE_API
